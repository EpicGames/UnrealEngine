// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Platforms/VectorVMPlatformBase.h"

#if PLATFORM_CPU_ARM_FAMILY

#define VVM_pshufb(Src, Mask) vqtbl1q_u8(Src, Mask)

static VM_FORCEINLINE void VVM_floatToHalf(void* output, float const* input)
{
	float16x4_t out0 = vcvt_f16_f32(vld1q_f32(input + 0));
	vst1_s16((int16_t*)((char*)output + 0), out0);
}


VM_FORCEINLINE VectorRegister4i VVMIntRShift(VectorRegister4i v0, VectorRegister4i v1)
{
	VectorRegister4i res = vshlq_u32(v0, vmulq_s32(v1, vdupq_n_s32(-1)));
	return res;
}

VM_FORCEINLINE VectorRegister4i VVMIntLShift(VectorRegister4i v0, VectorRegister4i v1)
{
	VectorRegister4i res = vshlq_u32(v0, v1);
	return res;
}

#endif // PLATFORM_CPU_ARM_FAMILY