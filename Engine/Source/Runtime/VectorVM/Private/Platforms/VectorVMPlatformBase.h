// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "VectorVM.h"
#include "Math/VectorRegister.h"

struct FVVM_VUI4
{
	union
	{
		VectorRegister4i v;
		uint32 u4[4];
		int32 i4[4];
	};
};



