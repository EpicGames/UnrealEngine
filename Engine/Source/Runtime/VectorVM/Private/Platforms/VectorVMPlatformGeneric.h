// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Platforms/VectorVMPlatformBase.h"

#if PLATFORM_CPU_X86_FAMILY || defined(__SSE3__)

#define VVM_pshufb(Src, Mask) _mm_shuffle_epi8(Src, Mask)
  // Fabian's round-to-nearest-even float to half
static void VVM_floatToHalf(void* output, float const* input)
{
	static const MS_ALIGN(16) unsigned int mask_sign[4]         GCC_ALIGN(16) = { 0x80000000u, 0x80000000u, 0x80000000u, 0x80000000u };
	static const MS_ALIGN(16)          int c_f16max[4]          GCC_ALIGN(16) = { (127 + 16) << 23, (127 + 16) << 23, (127 + 16) << 23, (127 + 16) << 23 }; // all FP32 values >=this round to +inf
	static const MS_ALIGN(16)          int c_nanbit[4]          GCC_ALIGN(16) = { 0x200, 0x200, 0x200, 0x200 };
	static const MS_ALIGN(16)          int c_infty_as_fp16[4]   GCC_ALIGN(16) = { 0x7c00, 0x7c00, 0x7c00, 0x7c00 };
	static const MS_ALIGN(16)          int c_min_normal[4]      GCC_ALIGN(16) = { (127 - 14) << 23, (127 - 14) << 23, (127 - 14) << 23, (127 - 14) << 23 }; // smallest FP32 that yields a normalized FP16
	static const MS_ALIGN(16)          int c_subnorm_magic[4]   GCC_ALIGN(16) = { ((127 - 15) + (23 - 10) + 1) << 23, ((127 - 15) + (23 - 10) + 1) << 23, ((127 - 15) + (23 - 10) + 1) << 23, ((127 - 15) + (23 - 10) + 1) << 23 };
	static const MS_ALIGN(16)          int c_normal_bias[4]     GCC_ALIGN(16) = { 0xfff - ((127 - 15) << 23), 0xfff - ((127 - 15) << 23), 0xfff - ((127 - 15) << 23), 0xfff - ((127 - 15) << 23) }; // adjust exponent and add mantissa rounding


	__m128  f = _mm_loadu_ps(input);
	__m128  msign = _mm_castsi128_ps(*(VectorRegister4i*)mask_sign);
	__m128  justsign = _mm_and_ps(msign, f);
	__m128  absf = _mm_xor_ps(f, justsign);
	__m128i absf_int = _mm_castps_si128(absf); // the cast is "free" (extra bypass latency, but no thruput hit)
	__m128i f16max = *(VectorRegister4i*)c_f16max;
	__m128  b_isnan = _mm_cmpunord_ps(absf, absf); // is this a NaN?
	__m128i b_isregular = _mm_cmpgt_epi32(f16max, absf_int); // (sub)normalized or special?
	__m128i nanbit = _mm_and_si128(_mm_castps_si128(b_isnan), *(VectorRegister4i*)c_nanbit);
	__m128i inf_or_nan = _mm_or_si128(nanbit, *(VectorRegister4i*)c_infty_as_fp16); // output for specials

	__m128i min_normal = *(VectorRegister4i*)c_min_normal;
	__m128i b_issub = _mm_cmpgt_epi32(min_normal, absf_int);

	// "result is subnormal" path
	__m128  subnorm1 = _mm_add_ps(absf, _mm_castsi128_ps(*(VectorRegister4i*)c_subnorm_magic)); // magic value to round output mantissa
	__m128i subnorm2 = _mm_sub_epi32(_mm_castps_si128(subnorm1), *(VectorRegister4i*)c_subnorm_magic); // subtract out bias

	// "result is normal" path
	__m128i mantoddbit = _mm_slli_epi32(absf_int, 31 - 13); // shift bit 13 (mantissa LSB) to sign
	__m128i mantodd = _mm_srai_epi32(mantoddbit, 31); // -1 if FP16 mantissa odd, else 0

	__m128i round1 = _mm_add_epi32(absf_int, *(VectorRegister4i*)c_normal_bias);
	__m128i round2 = _mm_sub_epi32(round1, mantodd); // if mantissa LSB odd, bias towards rounding up (RTNE)
	__m128i normal = _mm_srli_epi32(round2, 13); // rounded result

	// combine the two non-specials
	__m128i nonspecial = _mm_or_si128(_mm_and_si128(subnorm2, b_issub), _mm_andnot_si128(b_issub, normal));

	// merge in specials as well
	__m128i joined = _mm_or_si128(_mm_and_si128(nonspecial, b_isregular), _mm_andnot_si128(b_isregular, inf_or_nan));

	__m128i sign_shift = _mm_srai_epi32(_mm_castps_si128(justsign), 16);
	__m128i res = _mm_or_si128(joined, sign_shift);

	res = _mm_packs_epi32(res, res);
	_mm_storeu_si64(output, res);
}

VM_FORCEINLINE VectorRegister4i VVMIntRShift(VectorRegister4i v0, VectorRegister4i v1)
{
	uint32* v0_4 = (uint32*)&v0;
	uint32* v1_4 = (uint32*)&v1;

	FVVM_VUI4 res;

	res.u4[0] = v0_4[0] >> v1_4[0];
	res.u4[1] = v0_4[1] >> v1_4[1];
	res.u4[2] = v0_4[2] >> v1_4[2];
	res.u4[3] = v0_4[3] >> v1_4[3];

	return res.v;
}

VM_FORCEINLINE VectorRegister4i VVMIntLShift(VectorRegister4i v0, VectorRegister4i v1)
{
	uint32* v0_4 = (uint32*)&v0;
	uint32* v1_4 = (uint32*)&v1;

	FVVM_VUI4 res;

	res.u4[0] = v0_4[0] << v1_4[0];
	res.u4[1] = v0_4[1] << v1_4[1];
	res.u4[2] = v0_4[2] << v1_4[2];
	res.u4[3] = v0_4[3] << v1_4[3];

	return res.v;
}

#endif // PLATFORM_CPU_X86_FAMILY || defined(__SSE3__)