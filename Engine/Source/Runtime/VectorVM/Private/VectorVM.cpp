// Copyright Epic Games, Inc. All Rights Reserved.

#include "VectorVM.h"

#include "Async/ParallelFor.h"
#include "Modules/ModuleManager.h"
#include "Stats/Stats.h"
#include "UObject/Class.h"
#include "VectorVMTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(VectorVM)

IMPLEMENT_MODULE(FDefaultModuleImpl, VectorVM);

DECLARE_STATS_GROUP(TEXT("VectorVM"), STATGROUP_VectorVM, STATCAT_Advanced);
DEFINE_LOG_CATEGORY_STATIC(LogVectorVM, All, All);

namespace VectorVM
{

uint8 GetNumOpCodes()
{
	return (uint8)EVectorVMOp::NumOpcodes;
}

#if WITH_EDITOR
UEnum* VectorVMOperandEnum = nullptr;

#define VVM_OP_XM(n, ...) #n,
static const TArray<FString> VVM_OP_NAMES
{
	VVM_OP_XM_LIST
};
#undef VVM_OP_XM

FString GetOpName(EVectorVMOp Op)
{
	if (VVM_OP_NAMES.IsEmpty())
	{
		return FString();
	}

	const int32 OpIndex = static_cast<int32>(Op);
	return VVM_OP_NAMES.IsValidIndex(OpIndex) ? VVM_OP_NAMES[OpIndex] : VVM_OP_NAMES[0];
}

FString GetOperandLocationName(EVectorVMOperandLocation Location)
{
	check(VectorVMOperandEnum);

	FString LocStr = VectorVMOperandEnum->GetNameByValue((uint8)Location).ToString();
	int32 LastIdx = 0;
	LocStr.FindLastChar(TEXT(':'), LastIdx);
	return LocStr.RightChop(LastIdx + 1);
}
#endif

uint8 CreateSrcOperandMask(EVectorVMOperandLocation Type0, EVectorVMOperandLocation Type1, EVectorVMOperandLocation Type2)
{
	constexpr uint8 OP_REGISTER = 0;
	constexpr uint8 OP0_CONST = (1 << 0);
	constexpr uint8 OP1_CONST = (1 << 1);
	constexpr uint8 OP2_CONST = (1 << 2);

	return
		(Type0 == EVectorVMOperandLocation::Constant ? OP0_CONST : OP_REGISTER) |
		(Type1 == EVectorVMOperandLocation::Constant ? OP1_CONST : OP_REGISTER) |
		(Type2 == EVectorVMOperandLocation::Constant ? OP2_CONST : OP_REGISTER);
}

void Init()
{
	static bool Inited = false;
	if (Inited == false)
	{
#if WITH_EDITOR
		VectorVMOperandEnum = StaticEnum<EVectorVMOperandLocation>();
#endif

		Inited = true;
	}
}

} // VectorVM
