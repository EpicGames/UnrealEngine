// Copyright Epic Games, Inc. All Rights Reserved.

#include "VectorVMBridge.h"

#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"
#include "VectorVMEditor.h"
#include "VectorVMOptimizer.h"
#include "VectorVMTypes.h"

namespace VectorVM::Bridge
{

struct FRuntimeContextData
{
	FRuntimeContextData(const Runtime::FVectorVMRuntimeContext& Context)
	: NumBytecodeBytes(Context.NumBytecodeBytes)
	, MaxOutputDataSet(Context.MaxOutputDataSet)
	, NumConstsAlloced(Context.NumConstsAlloced)
	, NumTempRegisters(Context.NumTempRegisters)
	, NumConstsRemapped(Context.NumConstsRemapped)
	, NumInputsRemapped(Context.NumInputsRemapped)
	, NumNoAdvanceInputs(Context.NumNoAdvanceInputs)
	, NumInputDataSets(Context.NumInputDataSets)
	, NumOutputsRemapped(Context.NumOutputsRemapped)
	, NumOutputInstructions(Context.NumOutputInstructions)
	, NumExtFns(Context.NumExtFns)
	, MaxExtFnRegisters(Context.MaxExtFnRegisters)
	, NumDummyRegsReq(Context.NumDummyRegsReq)
	, MaxExtFnUsed(Context.MaxExtFnUsed)
	, Flags(Context.Flags)
	, HashId(Context.HashId)
	{}

	FRuntimeContextData(TConstArrayView<uint8> ContextData)
	{
		FMemoryReaderView Ar(ContextData);
		Ar << *this;
	}

	void CopyToContext(Runtime::FVectorVMRuntimeContext& Context) const
	{
		Context.NumBytecodeBytes = NumBytecodeBytes;
		Context.MaxOutputDataSet = MaxOutputDataSet;
		Context.NumConstsAlloced = NumConstsAlloced;
		Context.NumTempRegisters = NumTempRegisters;
		Context.NumConstsRemapped = NumConstsRemapped;
		Context.NumInputsRemapped = NumInputsRemapped;
		Context.NumNoAdvanceInputs = NumNoAdvanceInputs;
		Context.NumInputDataSets = NumInputDataSets;
		Context.NumOutputsRemapped = NumOutputsRemapped;
		Context.NumOutputInstructions = NumOutputInstructions;
		Context.NumExtFns = NumExtFns;
		Context.MaxExtFnRegisters = MaxExtFnRegisters;
		Context.NumDummyRegsReq = NumDummyRegsReq;
		Context.MaxExtFnUsed = MaxExtFnUsed;
		Context.Flags = Flags;
		Context.HashId = HashId;
	}

	friend FArchive& operator<<(FArchive& Ar, FRuntimeContextData& ContextInfo);

	uint32 NumBytecodeBytes;
	uint32 MaxOutputDataSet;
	uint16 NumConstsAlloced;
	uint32 NumTempRegisters;
	uint16 NumConstsRemapped;
	uint16 NumInputsRemapped;
	uint16 NumNoAdvanceInputs;
	uint16 NumInputDataSets;
	uint16 NumOutputsRemapped;
	uint16 NumOutputInstructions;
	uint32 NumExtFns;
	uint32 MaxExtFnRegisters;
	uint32 NumDummyRegsReq;
	int32 MaxExtFnUsed;
	uint32 Flags;
	uint64 HashId;
};

FArchive& operator<<(FArchive& Ar, FRuntimeContextData& ContextInfo)
{
	Ar << ContextInfo.NumBytecodeBytes;
	Ar << ContextInfo.MaxOutputDataSet;
	Ar << ContextInfo.NumConstsAlloced;
	Ar << ContextInfo.NumTempRegisters;
	Ar << ContextInfo.NumConstsRemapped;
	Ar << ContextInfo.NumInputsRemapped;
	Ar << ContextInfo.NumNoAdvanceInputs;
	Ar << ContextInfo.NumInputDataSets;
	Ar << ContextInfo.NumOutputsRemapped;
	Ar << ContextInfo.NumOutputInstructions;
	Ar << ContextInfo.NumExtFns;
	Ar << ContextInfo.MaxExtFnRegisters;
	Ar << ContextInfo.NumDummyRegsReq;
	Ar << ContextInfo.MaxExtFnUsed;
	Ar << ContextInfo.Flags;
	Ar << ContextInfo.HashId;

	return Ar;
}

struct FContextInfoLayout
{
	FContextInfoLayout(const FRuntimeContextData& Context)
		: PropertySize(sizeof(Context))
		, BytecodeSize(Context.NumBytecodeBytes)
		, ConstRemapSize(Context.NumConstsRemapped * sizeof(uint16))
		, InputRemapSize(Context.NumInputsRemapped * sizeof(uint16))
		, InputDataSetOffsetsSize(Context.NumInputDataSets * 8 * sizeof(uint16))
		, OutputRemapDataSetIdxSize(Context.NumOutputsRemapped * sizeof(uint8))
		, OutputRemapDataTypeSize(Context.NumOutputsRemapped * sizeof(uint16))
		, OutputRemapDstSize(Context.NumOutputsRemapped * sizeof(uint16))
		, ExtFnSize(Context.NumExtFns * sizeof(FVectorVMExtFunctionData))
		, PropertyOffset(0)
		, BytecodeOffset(Align(PropertyOffset + PropertySize, 16))
		, ConstRemapOffset(Align(BytecodeOffset + BytecodeSize, 16))
		, InputRemapOffset(Align(ConstRemapOffset + ConstRemapSize, 16))
		, InputDataSetOffsetsOffset(Align(InputRemapOffset + InputRemapSize, 16))
		, OutputRemapDataSetIdxOffset(Align(InputDataSetOffsetsOffset + InputDataSetOffsetsSize, 16))
		, OutputRemapDataTypeOffset(Align(OutputRemapDataSetIdxOffset + OutputRemapDataSetIdxSize, 16))
		, OutputRemapDstOffset(Align(OutputRemapDataTypeOffset + OutputRemapDataTypeSize, 16))
		, ExtFnOffset(Align(OutputRemapDstOffset + OutputRemapDstSize, 16))
		, TotalSize(Align(ExtFnOffset + ExtFnSize, 16))
	{
	}

	const uint32 PropertySize;
	const uint32 BytecodeSize;
	const uint32 ConstRemapSize;
	const uint32 InputRemapSize;
	const uint32 InputDataSetOffsetsSize;
	const uint32 OutputRemapDataSetIdxSize;
	const uint32 OutputRemapDataTypeSize;
	const uint32 OutputRemapDstSize;
	const uint32 ExtFnSize;

	const size_t PropertyOffset;
	const size_t BytecodeOffset;
	const size_t ConstRemapOffset;
	const size_t InputRemapOffset;
	const size_t InputDataSetOffsetsOffset;
	const size_t OutputRemapDataSetIdxOffset;
	const size_t OutputRemapDataTypeOffset;
	const size_t OutputRemapDstOffset;
	const size_t ExtFnOffset;
	const size_t TotalSize;
};

#if WITH_EDITORONLY_DATA
void FreezeOptimizerContext(const Optimizer::FVectorVMOptimizerContext& Context, TArray<uint8>& ContextData)
{
	FRuntimeContextData RuntimeData(Context);
	FContextInfoLayout Layout(RuntimeData);

	ContextData.SetNumZeroed(Layout.TotalSize);
	uint8* BufferData = ContextData.GetData();

	FMemoryWriter Ar(ContextData);
	Ar << RuntimeData;

	FMemory::Memcpy(BufferData + Layout.BytecodeOffset, Context.OutputBytecode, Layout.BytecodeSize);
	FMemory::Memcpy(BufferData + Layout.ConstRemapOffset, Context.ConstRemap[1], Layout.ConstRemapSize);
	FMemory::Memcpy(BufferData + Layout.InputRemapOffset, Context.InputRemapTable, Layout.InputRemapSize);
	FMemory::Memcpy(BufferData + Layout.InputDataSetOffsetsOffset, Context.InputDataSetOffsets, Layout.InputDataSetOffsetsSize);
	FMemory::Memcpy(BufferData + Layout.OutputRemapDataSetIdxOffset, Context.OutputRemapDataSetIdx, Layout.OutputRemapDataSetIdxSize);
	FMemory::Memcpy(BufferData + Layout.OutputRemapDataTypeOffset, Context.OutputRemapDataType, Layout.OutputRemapDataTypeSize);
	FMemory::Memcpy(BufferData + Layout.OutputRemapDstOffset, Context.OutputRemapDst, Layout.OutputRemapDstSize);
	FMemory::Memcpy(BufferData + Layout.ExtFnOffset, Context.ExtFnTable, Layout.ExtFnSize);

	// external function pointers
	FVectorVMExtFunctionData* ExtFunctionTable = reinterpret_cast<FVectorVMExtFunctionData*>(BufferData + Layout.ExtFnOffset);
	for (uint32 ExtFunctionIt = 0; ExtFunctionIt < Context.NumExtFns; ++ExtFunctionIt)
	{
		ExtFunctionTable[ExtFunctionIt].Function = nullptr;
	}

}
#endif // WITH_EDITORONLY_DATA

void ThawRuntimeContext(TConstArrayView<uint8> ContextData, Runtime::FVectorVMRuntimeContext& Context)
{
	FMemory::Memzero(Context);

	FRuntimeContextData RuntimeData(ContextData);

	FContextInfoLayout Layout(RuntimeData);

	RuntimeData.CopyToContext(Context);
	const uint8* BufferData = ContextData.GetData();
	Context.OutputBytecode = const_cast<uint8*>(reinterpret_cast<const uint8*>(BufferData + Layout.BytecodeOffset));
	Context.ConstRemap[1] = const_cast<uint16*>(reinterpret_cast<const uint16*>(BufferData + Layout.ConstRemapOffset));
	Context.InputRemapTable = const_cast<uint16*>(reinterpret_cast<const uint16*>(BufferData + Layout.InputRemapOffset));
	Context.InputDataSetOffsets = const_cast<uint16*>(reinterpret_cast<const uint16*>(BufferData + Layout.InputDataSetOffsetsOffset));
	Context.OutputRemapDataSetIdx = const_cast<uint8*>(reinterpret_cast<const uint8*>(BufferData + Layout.OutputRemapDataSetIdxOffset));
	Context.OutputRemapDataType = const_cast<uint16*>(reinterpret_cast<const uint16*>(BufferData + Layout.OutputRemapDataTypeOffset));
	Context.OutputRemapDst = const_cast<uint16*>(reinterpret_cast<const uint16*>(BufferData + Layout.OutputRemapDstOffset));
	Context.ExtFnTable = const_cast<FVectorVMExtFunctionData*>(reinterpret_cast<const FVectorVMExtFunctionData*>(BufferData + Layout.ExtFnOffset));
}

} // VectorVM::Bridge