// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "VectorVM.h"
#include "VectorVMTypes.h"

#define VVM_OP_CAT_XM_LIST    \
	VVM_OP_CAT_XM(Input)      \
	VVM_OP_CAT_XM(Output)     \
	VVM_OP_CAT_XM(Op)         \
	VVM_OP_CAT_XM(ExtFnCall)  \
	VVM_OP_CAT_XM(IndexGen)   \
	VVM_OP_CAT_XM(RWBuffer)   \
	VVM_OP_CAT_XM(Stat)       \
	VVM_OP_CAT_XM(Other)

#define VVM_OP_CAT_XM(Category, ...) Category,
enum class EVectorVMOpCategory : uint8 {
	VVM_OP_CAT_XM_LIST
	MAX
};
#undef VVM_OP_CAT_XM

static const EVectorVMOpCategory VVM_OP_CATEGORIES[] = {
#define VVM_OP_XM(op, cat, ...) EVectorVMOpCategory::cat,
	VVM_OP_XM_LIST
#undef VVM_OP_XM
};

#if WITH_EDITORONLY_DATA

namespace VectorVM::Optimizer
{

enum
{
	VVM_RT_TEMPREG,
	VVM_RT_CONST,
	VVM_RT_INPUT,
	VVM_RT_OUTPUT,
	VVM_RT_INVALID
};

struct FVectorVMOptimizerContext;

//prototypes for optimizer
typedef uint32 (VectorVMOptimizerErrorCallback)     (FVectorVMOptimizerContext* OptimizeContext, uint32 ErrorFlags);	//return new error flags

//Optimization
struct FVectorVMOptimizeInstruction
{
	EVectorVMOp         OpCode;
	EVectorVMOpCategory OpCat;
	uint32              PtrOffsetInOrigBytecode;
	uint32              PtrOffsetInOptimizedBytecode;
	int                 Index; //initial index.  Instructions are moved around and removed and dependency chains are created based on index, so we need to store this.
	int					InsMergedIdx; //if not -1, then the instruction index that this is merged with.  Instructions with a set InsMergedIdx are not written to the final bytecode
	int                 OutputMergeIdx[2]; //if not -1 then this instruction writes directly to an output, not a temp register
	uint16              RegPtrOffset;
	int                 NumInputRegisters;
	int                 NumOutputRegisters;
	union
	{
		struct
		{
			uint16 DataSetIdx;
			uint16 InputIdx;
		} Input;
		struct
		{
			uint16 DataSetIdx;
			uint16 DstRegIdx;
			int    MergeIdx; //if not -1 then this instruction index is merged with an output or an op, if it's -2 then it's already been taken care of
			uint16 SerialIdx;
		} Output;
		struct
		{

		} Op;
		struct
		{
			uint16 DataSetIdx;
		} IndexGen;
		struct
		{
			uint16 ExtFnIdx;
			uint16 NumInputs;
			uint16 NumOutputs;
		} ExtFnCall;
		struct
		{
			uint16 DataSetIdx;
		} RWBuffer;
		struct
		{
			uint16 ID;
		} Stat;
		struct
		{
		} Other;
	};
};

enum EVectorVMOptimizeError
{
	VVMOptErr_OutOfMemory           = 1 << 0,
	VVMOptErr_Overflow			    = 1 << 1,
	VVMOptErr_Bytecode              = 1 << 2,
	VVMOptErr_RegisterUsage         = 1 << 3,
	VVMOptErr_ConstRemap            = 1 << 4,
	VVMOptErr_Instructions          = 1 << 5,
	VVMOptErr_InputMergeBuffer      = 1 << 6,
	VVMOptErr_InstructionReOrder    = 1 << 7,
	VVMOptErr_SSARemap              = 1 << 8,
	VVMOptErr_OptimizedBytecode     = 1 << 9,
	VVMOptErr_ExternalFunction      = 1 << 10,
	VVMOptErr_RedundantInstruction  = 1 << 11,

	VVMOptErr_Fatal                 = 1 << 31
};

typedef void* (VectorVMReallocFn)(void* Ptr, size_t NumBytes, const char* Filename, int LineNumber);
typedef void   (VectorVMFreeFn)(void* Ptr, const char* Filename, int LineNumber);

struct FVectorVMOptimizerContext : public VectorVM::Runtime::FVectorVMRuntimeContext
{
	~FVectorVMOptimizerContext();

	struct
	{
		VectorVMReallocFn *               ReallocFn = nullptr;
		VectorVMFreeFn *                  FreeFn = nullptr;
		const char *                      ScriptName = nullptr;
	} Init;                                                    //Set this stuff when calling Optimize()

	struct
	{
		uint32                            Flags = 0;               //zero is good
		uint32                            Line = 0;
		VectorVMOptimizerErrorCallback*   CallbackFn = nullptr;          //set this to get a callback whenever there's an error
	} Error;

	struct
	{
		FVectorVMOptimizeInstruction*     Instructions = nullptr;
		uint8*                            RegisterUsageType = nullptr;
		uint16*                           RegisterUsageBuffer = nullptr;
		uint16*                           SSARegisterUsageBuffer = nullptr;
		uint16*                           ParentInstructionIdx = nullptr;
		uint32                            NumInstructions = 0;
		uint32                            NumInstructionsAlloced = 0;
		uint32                            NumRegistersUsed = 0;
	} Intermediate;                                           //these are freed and NULL after optimize() unless SaveIntermediateData is true when calling OptimizeVectorVMScript
};

}

#endif
