// Copyright Epic Games, Inc. All Rights Reserved.

#include "VectorVMRuntime.h"

#include "HAL/IConsoleManager.h"
#include "Math/RandomStream.h"
#include "VectorVM.h"
#include "VectorVMBridge.h"
#include "VectorVMTypes.h"

#if PLATFORM_CPU_X86_FAMILY || defined(__SSE3__)
#include "Platforms/VectorVMPlatformGeneric.h"
#elif PLATFORM_CPU_ARM_FAMILY
#include "Platforms/VectorVMPlatformARM.h"
#endif

#define VVM_MIN(a, b)               ((a) < (b) ? (a) : (b))
#define VVM_MAX(a, b)               ((a) > (b) ? (a) : (b))
#define VVM_CLAMP(v, min, max)      ((v) < (min) ? (min) : ((v) < (max) ? (v) : (max)))
#define VVM_ALIGN(num, alignment)   (((size_t)(num) + (alignment) - 1) & ~((alignment) - 1))
#define VVM_ALIGN_4(num)            (((size_t)(num) + 3) & ~3)
#define VVM_ALIGN_8(num)            (((size_t)(num) + 7) & ~7)
#define VVM_ALIGN_16(num)           (((size_t)(num) + 15) & ~15)
#define VVM_ALIGN_32(num)           (((size_t)(num) + 31) & ~31)
#define VVM_ALIGN_64(num)           (((size_t)(num) + 63) & ~63)

#define VVM_PTR_ALIGN    VVM_ALIGN_16
#define VVM_REG_SIZE     sizeof(FVecReg)

namespace VectorVM::Runtime
{

void *VVMDefaultRealloc(void *Ptr, size_t NumBytes, const char *Filename, int LineNumber)
{
	return FMemory::Realloc(Ptr, NumBytes);
}

void VVMDefaultFree(void *Ptr, const char *Filename, int LineNumber)
{
	return FMemory::Free(Ptr);
}

//cvar
static int32 GVVMPageSizeInKB = 64;
static FAutoConsoleVariableRef CVarVVMPageSizeInKB(
	TEXT("vm.PageSizeInKB"),
	GVVMPageSizeInKB,
	TEXT("Minimum allocation per VM instance.  There are 64 of these, so multiply GVVMPageSizeInKB * 64 * 1024 to get total number of bytes used by the VVM\n"),
	ECVF_ReadOnly 
);

struct FVectorVMBatchState
{
	MS_ALIGN(16) FVecReg* RegisterData GCC_ALIGN(16);
	struct
	{
		uint32* StartingOutputIdxPerDataSet;
		uint32* NumOutputPerDataSet;
		uint8** OutputMaskIdx; //these point to the BatchState's OutputMaskIdx
		struct
		{
			uint32** RegData;
			uint8* RegInc;
			FVecReg* DummyRegs;
		} ExtFnDecodedReg;
		int32* RandCounters; //used for external functions only.

		int                   ChunkIdx;
		int                   StartInstanceThisChunk;
		int                   NumInstancesThisChunk;
	} ChunkLocalData;

	uint8** RegPtrTable;    //not aligned because input pointers could be offest from the DataSetInfo.InstanceOffset, so we must assume every op is unaligned
	uint8* RegIncTable;    //0 for const, 1 for temp reg
	uint8* OutputMaskIdx;

	union
	{
		struct
		{
			VectorRegister4i          State[5]; //xorwor state for random/randomi instructions.  DIs use RandomStream.
			VectorRegister4i          Counters;
		};
	} RandState;
	FRandomStream             RandStream;
};

static_assert((sizeof(FVectorVMBatchState) & 0xF) == 0, "FVectorVMBatchState must be 16 byte aligned");

static const char VVM_RT_CHAR[] =
{
	'R', 'C', 'I', 'O', 'X'
};

#define VVM_CHUNK_FIXED_OVERHEAD_SIZE	512

//to avoid memset/memcpy when statically initializing sse variables
#define VVMSet_m128Const(Name, V)                static const MS_ALIGN(16) float VVMConstVec4_##Name##4[4]   GCC_ALIGN(16) = { V, V, V, V }
#define VVMSet_m128Const4(Name, V0, V1, V2, V3)  static const MS_ALIGN(16) float VVMConstVec4_##Name##4[4]   GCC_ALIGN(16) = { V0, V1, V2, V3 }
#define VVMSet_m128iConst(Name, V)               static const MS_ALIGN(16) uint32 VVMConstVec4_##Name##4i[4] GCC_ALIGN(16) = { V, V, V, V }
#define VVMSet_m128iConst4(Name, V0, V1, V2, V3) static const MS_ALIGN(16) uint32 VVMConstVec4_##Name##4i[4] GCC_ALIGN(16) = { V0, V1, V2, V3 }	/* equiv to setr */

#define VVM_m128Const(Name)  (*(VectorRegister4f *)&(VVMConstVec4_##Name##4))
#define VVM_m128iConst(Name) (*(VectorRegister4i *)&(VVMConstVec4_##Name##4i))

VVMSet_m128Const(   One             , 1.f);
VVMSet_m128Const(   NegativeOne     , -1.f);
VVMSet_m128Const(   OneHalf         , 0.5f);
VVMSet_m128Const(   Epsilon         , 1.e-8f);
VVMSet_m128Const(   HalfPi          , 3.14159265359f * 0.5f);
VVMSet_m128Const(   QuarterPi       , 3.14159265359f * 0.25f);
VVMSet_m128Const(   FastSinA        , 7.5894663844f);
VVMSet_m128Const(   FastSinB        , 1.6338434578f);
VVMSet_m128Const(   Log2            , 0.6931471806f);
VVMSet_m128Const(   OneOverLog2     , 1.4426950409f);
VVMSet_m128iConst(  FMask           , 0xFFFFFFFF);
VVMSet_m128iConst4( ZeroOneTwoThree , 0, 1, 2, 3);
VVMSet_m128iConst4( ZeroTwoFourSix  , 0, 2, 4, 6);
VVMSet_m128Const4(  ZeroOneTwoThree , 0.f, 1.f, 2.f, 3.f);
VVMSet_m128iConst(  RegOffsetMask   , 0x7FFF);
VVMSet_m128Const(   RegOneOverTwoPi , 1.f / 2.f / 3.14159265359f);
VVMSet_m128iConst(  AlmostTwoBits   , 0x3fffffff);

//the output instructions work on 4 32 bit values at a time.  The acquireindex instruction writes 1 byte for every
//4 instances.  The lower 4 bits correspond to whether the instance should be kept or not. 1 for keep, 0 for discard
//So for instance if we have 4 instances, and we're keeping the first and third, then acquireindex would write 0101
//This is 5 in base 10.  The output instructions would use the 5th element of this table to shuffle the values we
//want to write into the correct position.  On x64 this is used by the _mm_shuffle_epi8 intrinsic, 
//and vqtbl1q_u8 on ARM.  These are used via the macro VVM_pshufb (pshufb is the x64 instruction that corresponds to
//_mm_shuffle_epi8)

static const MS_ALIGN(16) uint8 VVM_PSHUFB_OUTPUT_TABLE[] GCC_ALIGN(16) = 
{
	0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF,     0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, // xxxx
	0x00, 0x01, 0x02, 0x03, 0xFF, 0xFF, 0xFF, 0xFF,     0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, // 0xxx
	0x04, 0x05, 0x06, 0x07, 0xFF, 0xFF, 0xFF, 0xFF,     0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, // 1xxx
	0x00, 0x01, 0x02, 0x03, 0x04, 0x05, 0x06, 0x07,     0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, // 01xx
	0x08, 0x09, 0x0A, 0x0B, 0xFF, 0xFF, 0xFF, 0xFF,     0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, // 2xxx
	0x00, 0x01, 0x02, 0x03, 0x08, 0x09, 0x0A, 0x0B,     0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, // 02xx
	0x04, 0x05, 0x06, 0x07, 0x08, 0x09, 0x0A, 0x0B,     0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, // 12xx
	0x00, 0x01, 0x02, 0x03, 0x04, 0x05, 0x06, 0x07,     0x08, 0x09, 0x0A, 0x0B, 0xFF, 0xFF, 0xFF, 0xFF, // 012x
	0x0C, 0x0D, 0x0E, 0x0F, 0xFF, 0xFF, 0xFF, 0xFF,     0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, // 3xxx
	0x00, 0x01, 0x02, 0x03, 0x0C, 0x0D, 0x0E, 0x0F,     0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, // 03xx
	0x04, 0x05, 0x06, 0x07, 0x0C, 0x0D, 0x0E, 0x0F,     0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, // 013x
	0x00, 0x01, 0x02, 0x03, 0x04, 0x05, 0x06, 0x07,     0x0C, 0x0D, 0x0E, 0x0F, 0xFF, 0xFF, 0xFF, 0xFF, // 013x
	0x08, 0x09, 0x0A, 0x0B, 0x0C, 0x0D, 0x0E, 0x0F,     0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, // 23xx
	0x00, 0x01, 0x02, 0x03, 0x08, 0x09, 0x0A, 0x0B,     0x0C, 0x0D, 0x0E, 0x0F, 0xFF, 0xFF, 0xFF, 0xFF, // 023x
	0x04, 0x05, 0x06, 0x07, 0x08, 0x09, 0x0A, 0x0B,     0x0C, 0x0D, 0x0E, 0x0F, 0xFF, 0xFF, 0xFF, 0xFF, // 123x
	0x00, 0x01, 0x02, 0x03, 0x04, 0x05, 0x06, 0x07,     0x08, 0x09, 0x0A, 0x0B, 0x0C, 0x0D, 0x0E, 0x0F	// 0123
};

static const MS_ALIGN(16) uint8 VVM_PSHUFB_OUTPUT_TABLE16[] GCC_ALIGN(16) =
{
	0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF,     0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 
	0x00, 0x01, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF,     0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 
	0x02, 0x03, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF,     0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 
	0x00, 0x01, 0x02, 0x03, 0xFF, 0xFF, 0xFF, 0xFF,     0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 
	0x04, 0x05, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF,     0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 
	0x00, 0x01, 0x04, 0x05, 0xFF, 0xFF, 0xFF, 0xFF,     0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 
	0x02, 0x03, 0x04, 0x05, 0xFF, 0xFF, 0xFF, 0xFF,     0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 
	0x00, 0x01, 0x02, 0x03, 0x04, 0x05, 0xFF, 0xFF,     0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 
	0x06, 0x07, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF,     0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 
	0x00, 0x01, 0x06, 0x07, 0xFF, 0xFF, 0xFF, 0xFF,     0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 
	0x02, 0x03, 0x06, 0x07, 0xFF, 0xFF, 0xFF, 0xFF,     0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 
	0x00, 0x01, 0x02, 0x03, 0x06, 0x07, 0xFF, 0xFF,     0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 
	0x04, 0x05, 0x06, 0x07, 0xFF, 0xFF, 0xFF, 0xFF,     0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 
	0x00, 0x01, 0x04, 0x05, 0x06, 0x07, 0xFF, 0xFF,     0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 
	0x02, 0x03, 0x04, 0x05, 0x06, 0x07, 0xFF, 0xFF,     0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 
	0x00, 0x01, 0x02, 0x03, 0x04, 0x05, 0x06, 0x07,     0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF, 0xFF
};

static const uint8 VVM_OUTPUT_ADVANCE_TABLE[] = 
{
	0,  4,  4,  8,
	4,  8,  8, 12,
	4,  8,  8, 12,
	8, 12, 12, 16
};

static const uint8 VVM_OUTPUT_ADVANCE_TABLE16[] = 
{
	0, 2, 2, 4,
	2, 4, 4, 6,
	2, 4, 4, 6,
	4, 6, 6, 8
};

VM_FORCEINLINE VectorRegister4i VVMf2i(VectorRegister4i v0)
{
	FVecReg u;
	u.i = v0;
	VectorRegister4i res = VectorFloatToInt(u.v);
	return res;
}

VM_FORCEINLINE VectorRegister4f VVMi2f(VectorRegister4f v0)
{
	FVecReg u;
	u.v = v0;
	VectorRegister4f res = VectorIntToFloat(u.i);
	return res;
}
#define VVMDebugBreakIf(expr)	if ((expr)) { PLATFORM_BREAK(); }

VM_FORCEINLINE uint16 float_to_half_fast3_rtne(uint32 f_in)
{
	uint16 h_out;
	FPlatformMath::StoreHalf(&h_out, *reinterpret_cast<float*>(&f_in));

	return h_out;
}

void VVMMemCpy(void *dst, void *src, size_t bytes)
{
	unsigned char *RESTRICT d      = (unsigned char *)dst;
	unsigned char *RESTRICT s      = (unsigned char *)src;
	unsigned char *RESTRICT s_end  = s + bytes;
	ptrdiff_t ofs_to_dest = d - s;
	if (bytes < 16)
	{
		if (bytes)
		{
			do
			{
				s[ofs_to_dest] = s[0];
				++s;
			} while (s < s_end);
		}
	}
	else
	{
		// do one unaligned to get us aligned for the stream out below
		VectorRegister4i i0 = VectorIntLoad(s);
		VectorIntStore(i0, d);
		s += 16 + 16 - ((size_t)d & 15); // S is 16 bytes ahead 
		while (s <= s_end)
		{
			i0 = VectorIntLoad(s - 16);
			VectorIntStoreAligned(i0, s - 16 + ofs_to_dest);
			s += 16;
		}
		// do one unaligned to finish the copy
		i0 = VectorIntLoad(s_end - 16);
		VectorIntStore(i0, s_end + ofs_to_dest - 16);
	}
}

void VVMMemSet32(void *dst, uint32 val, size_t num_vals)
{
	if (num_vals <= 4)
	{
		uint32 *dst32 = (uint32 *)dst;
		switch (num_vals)
		{
			case 4:		VectorIntStore(VectorIntSet1(val), dst); break;
			case 3:		dst32[2] = val;  //intentional fallthrough
			case 2:		dst32[1] = val;  //intentional fallthrough
			case 1:		dst32[0] = val;  //intentional fallthrough
			case 0:		break;
		}
	}
	else
	{
		VectorRegister4i v4 = VectorIntSet1(val);
		char *RESTRICT ptr = (char *)dst;
		char *RESTRICT end_ptr = ptr + num_vals * sizeof(val) - sizeof(v4);
		while (ptr < end_ptr) {
			VectorIntStore(v4, ptr);
			ptr += sizeof(v4);
		}
		VectorIntStore(v4, end_ptr);
	}
}

void VVMMemSet16(void *dst, uint16 val, size_t num_vals)
{
	VectorRegister4i Val4 = VectorIntLoad1_16(&val);
	if (num_vals <= 8)
	{
		uint16 *dst16 = (uint16 *)dst;
		switch (num_vals)
		{
			case 8:		VectorIntStore(Val4, dst16);
						break;
			case 7:		VectorIntStore_16(Val4, dst16);
						VectorIntStore_16(Val4, dst16 + 3);
						break;
			case 6:		VectorIntStore_16(Val4, dst16);
						VectorIntStore_16(Val4, dst16 + 2);
						break;
			case 5:		VectorIntStore_16(Val4, dst16);
						VectorIntStore_16(Val4, dst16 + 1);
						break;
			case 4:		VectorIntStore_16(Val4, dst16);
						break;
			case 3:		dst16[2] = val;  //intentional fallthrough
			case 2:		dst16[1] = val;  //intentional fallthrough
			case 1:		dst16[0] = val;  //intentional fallthrough
			case 0:		break;
		}
	}
	else
	{	
		char *RESTRICT ptr     = (char *)dst;
		char *RESTRICT end_ptr = ptr + num_vals * sizeof(val) - sizeof(Val4);
		while (ptr < end_ptr)
		{
			VectorIntStore(Val4, ptr);
			ptr += sizeof(Val4);
		}
		VectorIntStore(Val4, end_ptr);
	}
}

static void SetupBatchStatePtrs(FVectorVMExecContext *ExecCtx, FVectorVMBatchState *BatchState)
{
	uint8 *BatchDataPtr        = (uint8 *) VVM_ALIGN_64((size_t)BatchState + sizeof(FVectorVMBatchState));
	size_t NumPtrRegsInTable   = ExecCtx->VVMState->NumTempRegisters + ExecCtx->VVMState->NumConstBuffers + ExecCtx->VVMState->NumInputBuffers * 2 + ExecCtx->VVMState->NumOutputBuffers;
	uint32 NumLoops            = ExecCtx->Internal.MaxInstancesPerChunk >> 2;
	BatchState->RegisterData                                = (FVecReg *)BatchDataPtr;                  BatchDataPtr += VVM_REG_SIZE * ExecCtx->VVMState->NumTempRegisters * NumLoops;
	BatchState->RegPtrTable                                 = (uint8 **) BatchDataPtr;                  BatchDataPtr += NumPtrRegsInTable * sizeof(uint32 *);
	BatchState->RegIncTable                                 = (uint8 *)  BatchDataPtr;                  BatchDataPtr += NumPtrRegsInTable;
	BatchState->OutputMaskIdx                               = (uint8 *)  BatchDataPtr;                  BatchDataPtr += ExecCtx->VVMState->MaxOutputDataSet * NumLoops;
	BatchState->ChunkLocalData.StartingOutputIdxPerDataSet	= (uint32 *) BatchDataPtr;                  BatchDataPtr += ExecCtx->VVMState->ChunkLocalDataOutputIdxNumBytes;
	BatchState->ChunkLocalData.NumOutputPerDataSet          = (uint32 *) BatchDataPtr;                  BatchDataPtr += ExecCtx->VVMState->ChunkLocalNumOutputNumBytes;
	BatchState->ChunkLocalData.OutputMaskIdx                = (uint8 **) BatchDataPtr;                  BatchDataPtr += ExecCtx->VVMState->ChunkLocalOutputMaskIdxNumBytes;
	BatchState->ChunkLocalData.RandCounters = nullptr; //these get malloc'd separately if they're ever used... which they very rarely are

	for (uint32 i = 0; i < ExecCtx->VVMState->MaxOutputDataSet; ++i)
	{
		BatchState->ChunkLocalData.OutputMaskIdx[i] = BatchState->OutputMaskIdx + i * NumLoops;
	}

	{ //deal with the external function register decoding buffer
		size_t PtrBeforeExtFnDecodeReg = (size_t)BatchDataPtr;
		BatchState->ChunkLocalData.ExtFnDecodedReg.RegData       = (uint32 **)BatchDataPtr;  BatchDataPtr += sizeof(FVecReg *) * ExecCtx->VVMState->MaxExtFnRegisters;
		BatchState->ChunkLocalData.ExtFnDecodedReg.RegInc        = (uint8 *)BatchDataPtr;    BatchDataPtr += sizeof(uint8)     * ExecCtx->VVMState->MaxExtFnRegisters;
		BatchDataPtr = (uint8 *)VVM_PTR_ALIGN(BatchDataPtr);
		BatchState->ChunkLocalData.ExtFnDecodedReg.DummyRegs     = (FVecReg *)BatchDataPtr;  BatchDataPtr += sizeof(FVecReg)   * ExecCtx->VVMState->NumDummyRegsRequired;
	}
	size_t PtrStart               = (size_t)BatchState;
	size_t PtrAfterExtFnDecodeReg = (size_t)BatchDataPtr;
	check(PtrAfterExtFnDecodeReg - PtrStart <= ExecCtx->VVMState->BatchOverheadSize + ExecCtx->Internal.PerBatchRegisterDataBytesRequired);

	{ //build the register pointer table which contains pointers (in order) to:
		uint32 **TempRegPtr     = (uint32 **)BatchState->RegPtrTable;                      //1. Temp Registers
		uint32 **ConstBuffPtr   = TempRegPtr   + ExecCtx->VVMState->NumTempRegisters;      //2. Constant Buffers
		uint32 **InputPtr       = ConstBuffPtr + ExecCtx->VVMState->NumConstBuffers;       //3. Input Registers
		uint32 **OutputPtr      = InputPtr     + ExecCtx->VVMState->NumInputBuffers * 2;   //4. Output Buffers
		
		static_assert(sizeof(FVecReg) == 16);
		FMemory::Memset(BatchState->RegIncTable, sizeof(FVecReg), NumPtrRegsInTable);

		//temp regsiters
		for (uint32 i = 0; i < ExecCtx->VVMState->NumTempRegisters; ++i)
		{
			TempRegPtr[i] = (uint32 *)(BatchState->RegisterData + i * NumLoops);
		}
		//constant buffers
		for (uint32 i = 0; i < ExecCtx->VVMState->NumConstBuffers; ++i)
		{
			ConstBuffPtr[i] = (uint32 *)(ExecCtx->VVMState->ConstantBuffers + i);
			BatchState->RegIncTable[ExecCtx->VVMState->NumTempRegisters + i] = 0;
		}
		//inputs
		int NoAdvCounter = 0;
		for (uint32 i = 0; i < ExecCtx->VVMState->NumInputBuffers; ++i)
		{
			uint8 DataSetIdx      = ExecCtx->VVMState->InputMapCacheIdx[i];
			uint16 InputMapSrcIdx = ExecCtx->VVMState->InputMapCacheSrc[i];

			uint32 **DataSetInputBuffers = (uint32 **)ExecCtx->DataSets[DataSetIdx].InputRegisters.GetData();
			int32 InstanceOffset         = ExecCtx->DataSets[DataSetIdx].InstanceOffset;

			const bool bNoAdvanceInput = InputMapSrcIdx & 0x8000;
			const bool bHalfInput = InputMapSrcIdx & 0x4000;
			InputMapSrcIdx = InputMapSrcIdx & 0x3FFF;

			const int32 InputRegisterIndex = ExecCtx->VVMState->NumTempRegisters + ExecCtx->VVMState->NumConstBuffers + i;

			if (bNoAdvanceInput) //this is a noadvance input.  It points to data after the constant buffers
			{
				InputPtr[i] = (uint32 *)(ExecCtx->VVMState->ConstantBuffers + ExecCtx->VVMState->NumConstBuffers + NoAdvCounter);
				++NoAdvCounter;
				
				BatchState->RegIncTable[InputRegisterIndex] = 0; //no advance inputs... don't advance obviously

				if (bHalfInput) //half input (@TODO: has never been tested)
				{
					uint16 *Ptr = (uint16 *)DataSetInputBuffers[InputMapSrcIdx] + InstanceOffset;
					float val = FPlatformMath::LoadHalf(Ptr);
					VectorRegister4f InputVal = VectorSet1(val);
					VectorStore(InputVal, (float *)InputPtr[i]);
				}
				else
				{
					uint32 *Ptr = (uint32 *)DataSetInputBuffers[InputMapSrcIdx] + InstanceOffset;
					VectorRegister4i InputVal4 = VectorIntSet1(*Ptr);
					VectorIntStore(InputVal4, InputPtr[i]);
				}
			}
			else //regular input, point directly to the input buffer
			{
				const uint32 DataTypeStride = bHalfInput ? 2 : 4;
				const uint32 OffsetBytes = InstanceOffset * DataTypeStride;

				// Note that we don't update RegIncTable because it is handled by the op being invoked
				// (it will assume that the register is half as appropriate)

				InputPtr[i] = reinterpret_cast<uint32*>(
					reinterpret_cast<uint8*>(DataSetInputBuffers[InputMapSrcIdx]) + OffsetBytes);
			}

			//second copy of the "base" ptr so each chunk can start them at their correct starting offset
			InputPtr[i + ExecCtx->VVMState->NumInputBuffers] = InputPtr[i]; 
		}
		//outputs
		for (uint32 i = 0; i < ExecCtx->VVMState->NumOutputBuffers; ++i)
		{
			const uint8 DataSetIdx              = ExecCtx->VVMState->OutputRemapDataSetIdx[i];
			check(DataSetIdx < ExecCtx->DataSets.Num());

			const uint16 OutputDataType         = ExecCtx->VVMState->OutputRemapDataType[i];
			check(OutputDataType < UE_ARRAY_COUNT(FDataSetMeta::OutputRegisterTypeOffsets));

			const uint16 OutputMapDst           = ExecCtx->VVMState->OutputRemapDst[i];

			if (OutputMapDst == 0xFFFF)
			{
				OutputPtr[i] = nullptr;
			}
			else
			{
				const uint32 TypeOffset = ExecCtx->DataSets[DataSetIdx].OutputRegisterTypeOffsets[OutputDataType];
				const uint32 OutputBufferIdx = TypeOffset + OutputMapDst;
				const uint32 DataTypeStride = OutputDataType == 2 ? 2 : 4;
				const uint32 InstanceOffsetBytes = ExecCtx->DataSets[DataSetIdx].InstanceOffset * DataTypeStride;

				OutputPtr[i] = reinterpret_cast<uint32*>(ExecCtx->DataSets[DataSetIdx].OutputRegisters[OutputBufferIdx] + InstanceOffsetBytes);
			}
		}
	}
}

static void SetupRandStateForBatch(FVectorVMBatchState *BatchState)
{
	uint64 pcg_state = FPlatformTime::Cycles64();
	uint64 pcg_inc   = (((uint64)BatchState << 32) ^ 0XCAFEF00DD15EA5E5U) | 1;
	pcg_state ^= (FPlatformTime::Cycles64() << 32ULL);
	//use psuedo-pcg to setup a state for xorwow
	for (int i = 0; i < 5; ++i) //loop for xorwow internal state
	{
		MS_ALIGN(16) uint32 Values[4] GCC_ALIGN(16);
		for (int j = 0; j < 4; ++j)
		{
			uint64 old_state   = pcg_state;
			pcg_state          = old_state * 6364136223846793005ULL + pcg_inc;
			uint32 xor_shifted = (uint32)(((old_state >> 18U) ^ old_state) >> 27U);
			uint32 rot         = old_state >> 59U;
			Values[j]          = (xor_shifted >> rot) | (xor_shifted << ((0U - rot) & 31));
		}
		VectorIntStore(*(VectorRegister4i *)Values, BatchState->RandState.State + i);
	}
	BatchState->RandState.Counters = MakeVectorRegisterInt64(pcg_inc, pcg_state);
	BatchState->RandStream.GenerateNewSeed();
}

static VM_FORCEINLINE VectorRegister4i VVMXorwowStep(FVectorVMBatchState *BatchState)
{
	VectorRegister4i t = BatchState->RandState.State[4];
	VectorRegister4i s = BatchState->RandState.State[0];
	BatchState->RandState.State[4] = BatchState->RandState.State[3];
	BatchState->RandState.State[3] = BatchState->RandState.State[2];
	BatchState->RandState.State[2] = BatchState->RandState.State[1];
	BatchState->RandState.State[1] = s;
	t = VectorIntXor(t, VectorShiftRightImmLogical(t, 2));
	t = VectorIntXor(t, VectorShiftLeftImm(t, 1));
	t = VectorIntXor(t, VectorIntXor(s, VectorIntXor(s, VectorShiftLeftImm(s, 4))));
	BatchState->RandState.State[0] = t;
	BatchState->RandState.Counters = VectorIntAdd(BatchState->RandState.Counters, VectorIntSet1(362437));
	VectorRegister4i Result = VectorIntAdd(t, VectorIntLoad(&BatchState->RandState.Counters));
	return Result;
}

static void VVMBuildMapTableCaches(FVectorVMExecContext *ExecCtx)
{
	//constant buffers
	check(ExecCtx->ConstantTableCount <= 0xFF);
	check(ExecCtx->VVMState->NumConstBuffers <= 0xFFFF);
	for (uint32 i = 0; i < ExecCtx->VVMState->NumConstBuffers; ++i)
	{
		uint32 RemappedIdx = ExecCtx->VVMState->ConstRemapTable[i];
		uint32 ConstCountAcc = 0;
		for (int j = 0; j < ExecCtx->ConstantTableCount; ++j)
		{
			const uint32 NumDWords = (uint32)ExecCtx->ConstantTableNumBytes[j] >> 2;
			if (ConstCountAcc + NumDWords > RemappedIdx)
			{
				const uint32 *SrcConstArray = (uint32 *)ExecCtx->ConstantTableData[j];
				uint32 Idx = RemappedIdx - ConstCountAcc;
				check(Idx < 0xFFFF);
				ExecCtx->VVMState->ConstMapCacheIdx[i] = j;
				ExecCtx->VVMState->ConstMapCacheSrc[i] = (uint16)(Idx);
				break;
			}
			ConstCountAcc += NumDWords;
		}
	}

	//inputs
	if (ExecCtx->VVMState->NumInputDataSets > 0)
	{
		int InputCounter     = 0;
		int NumInputDataSets = VVM_MIN(ExecCtx->VVMState->NumInputDataSets, (uint32)ExecCtx->DataSets.Num()); //Niagara can pass in any amount of datasets, but we only care about the highest one actually used as determined by the optimizer
		for (int i = 0; i < NumInputDataSets; ++i)
		{
			uint32 **DataSetInputBuffers = (uint32 **)ExecCtx->DataSets[i].InputRegisters.GetData();

			//regular inputs: float, int and half
			for (int j = 0; j < 3; ++j)
			{
				int NumInputsThisType = ExecCtx->VVMState->InputDataSetOffsets[(i << 3) + j + 1] - ExecCtx->VVMState->InputDataSetOffsets[(i << 3) + j];
				int TypeOffset = ExecCtx->DataSets[i].InputRegisterTypeOffsets[j];
				for (int k = 0; k < NumInputsThisType; ++k)
				{
					int RemapIdx = ExecCtx->VVMState->InputDataSetOffsets[(i << 3) + j] + k;
					ExecCtx->VVMState->InputMapCacheIdx[InputCounter] = i;
					ExecCtx->VVMState->InputMapCacheSrc[InputCounter] = TypeOffset + ExecCtx->VVMState->InputRemapTable[RemapIdx] | (((j & 2) << 13));
					++InputCounter;
				}
			}

			if (ExecCtx->VVMState->InputDataSetOffsets[(i << 3) + 7] > 0)
			{
				//no advance inputs: float, int and half
				//no advance inputs point directly after the constant buffers
				for (int j = 0; j < 3; ++j)
				{
					int NumInputsThisType = ExecCtx->VVMState->InputDataSetOffsets[(i << 3) + j + 4] - ExecCtx->VVMState->InputDataSetOffsets[(i << 3) + j + 3];
					int TypeOffset = ExecCtx->DataSets[i].InputRegisterTypeOffsets[j];
					for (int k = 0; k < NumInputsThisType; ++k)
					{
						int RemapIdx     = ExecCtx->VVMState->InputDataSetOffsets[(i << 3) + j + 3] + k;
						ExecCtx->VVMState->InputMapCacheIdx[InputCounter] = i;
						ExecCtx->VVMState->InputMapCacheSrc[InputCounter] = (TypeOffset + ExecCtx->VVMState->InputRemapTable[RemapIdx]) | 0x8000 | (((j & 2) << 13)); //high bit is no advance input, 2nd high bit is whether it's half
						++InputCounter;
					}
				}
			}
		}
		check(InputCounter == ExecCtx->VVMState->NumInputBuffers);
	}
}

#define VVM_OUTPUT_FUNCTION_HEADER(CT_InputInsOutputTypeOpCode)                                              \
	uint32 *NumOutputPerDataSet         = BatchState->ChunkLocalData.NumOutputPerDataSet;                    \
	uint32 *StartingOutputIdxPerDataSet = BatchState->ChunkLocalData.StartingOutputIdxPerDataSet;            \
	uint8 **RegPtrTable                 = BatchState->RegPtrTable;                                           \
	uint8 *RegIncTable                  = BatchState->RegIncTable;                                           \
	uint8 **OutputMaskIdx               = BatchState->ChunkLocalData.OutputMaskIdx;                          \
	TArrayView<FDataSetMeta> DataSets   = ExecCtx->DataSets;                                                 \
	uint8  RegType            = (uint8)(CT_InputInsOutputTypeOpCode) - (uint8)EVectorVMOp::outputdata_float; \
	int NumOutputLoops        = InsPtr[0];                                                                   \
	uint8 DataSetIdx          = InsPtr[1];                                                                   \
	uint32 NumOutputInstances = NumOutputPerDataSet[DataSetIdx];                                             \
	uint32 RegTypeOffset      = DataSets[DataSetIdx].OutputRegisterTypeOffsets[RegType];                     \
	const uint16 * RESTRICT SrcIndices       = (uint16 *)(InsPtr + 2);                                       \
	const uint16 * RESTRICT DstIndices       = SrcIndices + NumOutputLoops;                                  \
	InsPtr += 3 + 4 * NumOutputLoops;

#define VVM_OUTPUT_FUNCTION_FOOTER return InsPtr

static const uint8 *VVM_Output32_from_16(const bool CT_MultipleLoops, const uint8 *InsPtr, FVectorVMBatchState *BatchState, FVectorVMExecContext *ExecCtx)
{
	VVM_OUTPUT_FUNCTION_HEADER(EVectorVMOp::outputdata_float); //-V501
	VVM_OUTPUT_FUNCTION_FOOTER;
}

static const uint8 *VVM_Output16_from_16(const bool CT_MultipleLoops, const uint8 *InsPtr, FVectorVMBatchState *BatchState, FVectorVMExecContext *ExecCtx)
{
	VVM_OUTPUT_FUNCTION_HEADER(EVectorVMOp::outputdata_half);
	if (CT_MultipleLoops)
	{
		if (NumOutputInstances == BatchState->ChunkLocalData.NumInstancesThisChunk) //all outputs written
		{ 
			for (int j = 0; j < NumOutputLoops; ++j)
			{
				int      SrcInc = RegIncTable[SrcIndices[j]];
				uint32 * SrcReg = (uint32 *)RegPtrTable[SrcIndices[j]];
				uint32 * DstReg = (uint32 *)RegPtrTable[DstIndices[j]];
				if (SrcReg != DstReg)
				{ //temp registers can be aliased to outputs
					if (SrcInc == 0) //setting from a constant
					{ 
						VVMMemSet16(DstReg, *SrcReg, NumOutputInstances);
					}
					else
					{
						VVMMemCpy(DstReg, SrcReg, sizeof(uint16) * NumOutputInstances);
					}
				}
			}
		}
		else if (NumOutputInstances > 0) //not all outputs are being written
		{
			for (int j = 0; j < NumOutputLoops; ++j)
			{
				int     SrcInc = RegIncTable[SrcIndices[j]];
				uint64 *DstReg = (uint64 *)RegPtrTable[DstIndices[j]];

				if (SrcInc == 0) //setting from a constant
				{ 
					uint64 Val              = *(uint64 *)RegPtrTable[SrcIndices[j]];
					uint64 *RESTRICT DstEnd = DstReg + NumOutputInstances;
					uint64 *RESTRICT Ptr    = DstReg;
					while (Ptr < DstEnd)
					{
						*Ptr = Val;
						Ptr++;
					}
				}
				else
				{
					int NumLoops = (int)((BatchState->ChunkLocalData.NumInstancesThisChunk + 3) & ~3) >> 2; //assumes 4-wide ops
					char * SrcPtr                 = (char *)RegPtrTable[SrcIndices[j]]; //src and dst can alias
					char * DstPtr                 = (char *)DstReg;
					uint8 * RESTRICT TblIdxPtr    = OutputMaskIdx[DataSetIdx];
					uint8 * RESTRICT TblIdxEndPtr = TblIdxPtr + NumLoops;
					while (TblIdxPtr < TblIdxEndPtr)
					{
						uint8 TblIdx = *TblIdxPtr++;
						VectorRegister4i Mask = ((VectorRegister4i *)VVM_PSHUFB_OUTPUT_TABLE16)[TblIdx];
						VectorRegister4i Src  = VectorIntLoad_16(SrcPtr);
						VectorRegister4i Val  = VVM_pshufb(Src, Mask);
						VectorIntStore_16(Val, DstPtr);
						SrcPtr += sizeof(uint16) * 4;
						DstPtr += VVM_OUTPUT_ADVANCE_TABLE16[TblIdx];
					}
				}
			}
		}
	}
	else
	{
		uint8 OutputMask = OutputMaskIdx[DataSetIdx][0];
		for (int j = 0; j < NumOutputLoops; ++j)
		{
			uint32 *SrcReg = (uint32 *)RegPtrTable[SrcIndices[j]];
			uint32 *DstReg = (uint32 *)RegPtrTable[DstIndices[j]];
			VectorRegister4i Mask = ((VectorRegister4i *)VVM_PSHUFB_OUTPUT_TABLE16)[OutputMask];
			VectorRegister4i Src  = VectorIntLoad(SrcReg);
			VectorRegister4i Val  = VVM_pshufb(Src, Mask);
			VectorIntStore_16(Val, DstReg);
		}
	}
	VVM_OUTPUT_FUNCTION_FOOTER;
}

static const uint8 *VVM_Output16(const bool CT_MultipleLoops, const uint8 *InsPtr, FVectorVMBatchState *BatchState, FVectorVMExecContext *ExecCtx)
{
	VVM_OUTPUT_FUNCTION_HEADER(EVectorVMOp::outputdata_half);
	
	if (CT_MultipleLoops)
	{
		if (NumOutputInstances == BatchState->ChunkLocalData.NumInstancesThisChunk) //all outputs written
		{
			for (int j = 0; j < NumOutputLoops; ++j)
			{
				int     SrcInc = RegIncTable[SrcIndices[j]];
				uint32 * RESTRICT SrcReg = (uint32 *)RegPtrTable[SrcIndices[j]];
				uint16 * RESTRICT DstReg = (uint16 *)RegPtrTable[DstIndices[j]];
				check((void *)SrcReg != (void *)DstReg); //half floats can't alias outputs
				if (SrcInc == 0) //setting from a constant
				{ 
					//constants are 32 bits so convert
					uint16 Val[8];
					VVM_floatToHalf(&Val, (float *)SrcReg);
					VVMMemSet16(DstReg, Val[0], NumOutputInstances);
				}
				else
				{
					uint16 * RESTRICT DstEnd = DstReg + NumOutputInstances; //this may go over the 16 byte boundary that's alloced if there's an offset... not sure!
					while (DstReg < DstEnd)
					{
						VVM_floatToHalf(DstReg, (float *)SrcReg);
						SrcReg += 4;
						DstReg += 4;
					}
				}
			}
		}
		else if (NumOutputInstances > 0) //not all outputs are being written
		{
			for (int j = 0; j < NumOutputLoops; ++j)
			{
				int   SrcInc = RegIncTable[SrcIndices[j]];
				char *DstReg = (char *)RegPtrTable[DstIndices[j]];
				if (SrcInc == 0) //setting from a constant
				{ 
					uint16 Val[8];
					VVM_floatToHalf(&Val, (float *)RegPtrTable[SrcIndices[j]]);
					VVMMemSet16(DstReg, Val[0], NumOutputInstances);
				}
				else
				{
					int NumLoops = (int)((BatchState->ChunkLocalData.NumInstancesThisChunk + 3) & ~3) >> 2; //assumes 4-wide ops
					char * SrcPtr                 = (char *)RegPtrTable[SrcIndices[j]]; //src and dst can alias
					char * DstPtr                 = DstReg;
					uint8 * RESTRICT TblIdxPtr    = OutputMaskIdx[DataSetIdx];
					uint8 * RESTRICT TblIdxEndPtr = TblIdxPtr + NumLoops;
					
					while (TblIdxPtr < TblIdxEndPtr)
					{
						uint8 TblIdx = *TblIdxPtr++;
						VectorRegister4i Mask = ((VectorRegister4i *)VVM_PSHUFB_OUTPUT_TABLE16)[TblIdx];
						VectorRegister4i Src  = VectorIntLoad((VectorRegister4i *)SrcPtr); //loading from a temp register and they're always aligned
						VectorRegister4i Val;
						FPlatformMath::VectorStoreHalf((uint16*) &Val, (float*)SrcPtr);
						VectorIntStore_16(VVM_pshufb(Val, Mask), DstPtr);
						SrcPtr += sizeof(VectorRegister4i);
						DstPtr += VVM_OUTPUT_ADVANCE_TABLE16[TblIdx];
					}
				}
			}
		}
	}
	else
	{
		uint8 OutputMask = OutputMaskIdx[DataSetIdx][0];
		for (int j = 0; j < NumOutputLoops; ++j)
		{
			float * SrcReg = (float *) RegPtrTable[SrcIndices[j]];
			uint16 *DstReg = (uint16 *)RegPtrTable[DstIndices[j]];
			//convert 4 values at once then shift them in place
			VectorRegister4i Mask = ((VectorRegister4i *)VVM_PSHUFB_OUTPUT_TABLE16)[OutputMask];
			VectorRegister4i HalfVals;
			FPlatformMath::VectorStoreHalf((uint16*) &HalfVals, SrcReg);
			VectorIntStore_16(VVM_pshufb(HalfVals, Mask), DstReg);
		}
	}
	VVM_OUTPUT_FUNCTION_FOOTER;
}

static const uint8 *VVM_Output32(const bool CT_MultipleLoops, const uint8 *InsPtr, FVectorVMBatchState *BatchState, FVectorVMExecContext *ExecCtx)
{
	VVM_OUTPUT_FUNCTION_HEADER(InsPtr[-1]);
	if (CT_MultipleLoops)
	{
		if (NumOutputInstances == BatchState->ChunkLocalData.NumInstancesThisChunk) //all outputs written
		{ 
			for (int j = 0; j < NumOutputLoops; ++j)
			{
				int      SrcInc = RegIncTable[SrcIndices[j]];
				uint32 * SrcReg = (uint32 *)RegPtrTable[SrcIndices[j]];
				uint32 * DstReg = (uint32 *)RegPtrTable[DstIndices[j]];
				if (SrcReg != DstReg)
				{ //temp registers can be aliased to outputs
					if (SrcInc == 0) //setting from a constant
					{ 
						VVMMemSet32(DstReg, *SrcReg, NumOutputInstances);
					}
					else
					{
						VVMMemCpy(DstReg, SrcReg, sizeof(uint32) * NumOutputInstances);
					}
				}
			}
		}
		else if (NumOutputInstances > 0) //not all outputs are being written
		{
			for (int j = 0; j < NumOutputLoops; ++j)
			{
				int   SrcInc = RegIncTable[SrcIndices[j]];
				char *DstReg = (char *)RegPtrTable[DstIndices[j]];
				if (SrcInc == 0) //setting from a constant
				{ 
					VectorRegister4i Val   = *(VectorRegister4i *)RegPtrTable[SrcIndices[j]];
					char * RESTRICT PtrEnd = DstReg + sizeof(uint32) * NumOutputInstances;
					char * RESTRICT Ptr    = DstReg;
					while (Ptr < PtrEnd)
					{
						VectorIntStore(Val, Ptr);
						Ptr += sizeof(VectorRegister4i);
					}
				}
				else
				{
					int NumLoops = (int)((BatchState->ChunkLocalData.NumInstancesThisChunk + 3) & ~3) >> 2; //assumes 4-wide ops
					char * SrcPtr                 = (char *)RegPtrTable[SrcIndices[j]]; //src and dst can alias
					char * DstPtr                 = DstReg;
					uint8 * RESTRICT TblIdxPtr    = OutputMaskIdx[DataSetIdx];
					uint8 * RESTRICT TblIdxEndPtr = TblIdxPtr + NumLoops;
					while (TblIdxPtr < TblIdxEndPtr)
					{
						
						uint8 TblIdx = *TblIdxPtr++;
						VectorRegister4i Mask = ((VectorRegister4i *)VVM_PSHUFB_OUTPUT_TABLE)[TblIdx];
						VectorRegister4i Src  = VectorIntLoad(SrcPtr);
						VectorRegister4i Val  = VVM_pshufb(Src, Mask);
						VectorIntStore(Val, DstPtr);
						SrcPtr += sizeof(VectorRegister4i);
						DstPtr += VVM_OUTPUT_ADVANCE_TABLE[TblIdx];
					}
				}
			}
		}
	}
	else
	{
		uint8 OutputMask = OutputMaskIdx[DataSetIdx][0];
		for (int j = 0; j < NumOutputLoops; ++j)
		{
			uint32 *SrcReg = (uint32 *)RegPtrTable[SrcIndices[j]];
			uint32 *DstReg = (uint32 *)RegPtrTable[DstIndices[j]];
			VectorRegister4i Mask = ((VectorRegister4i *)VVM_PSHUFB_OUTPUT_TABLE)[OutputMask];
			VectorRegister4i Src  = VectorIntLoad(SrcReg);
			VectorRegister4i Val  = VVM_pshufb(Src, Mask);
			VectorIntStore(Val, DstReg);
		}
	}
	VVM_OUTPUT_FUNCTION_FOOTER;
}

static const uint8 *VVM_acquireindex(const bool CT_MultipleLoops, const uint8 *InsPtr, FVectorVMBatchState *BatchState, FVectorVMExecContext *ExecCtx)
{
	uint32 *InputPtr          = (uint32 *)BatchState->RegPtrTable[((uint16 *)InsPtr)[0]];
	uint32 InputInc           = ((uint32)BatchState->RegIncTable[((uint16 *)InsPtr)[0]]) >> 2;
	uint8 DataSetIdx          = InsPtr[2];
	uint8 *OutputPtr          = BatchState->ChunkLocalData.OutputMaskIdx[DataSetIdx];
	uint32 NumOutputInstances = 0;
	if (CT_MultipleLoops)
	{
		int NumFullLoops          = BatchState->ChunkLocalData.NumInstancesThisChunk >> 2;
		int NumLeftoverInstances  = BatchState->ChunkLocalData.NumInstancesThisChunk - (NumFullLoops << 2);

		for (int i = 0; i < NumFullLoops; ++i)
		{
			uint32 idx = ((InputPtr[0] & (1 << 31)) >> 31) |
							((InputPtr[1] & (1 << 31)) >> 30) |
							((InputPtr[2] & (1 << 31)) >> 29) |
							((InputPtr[3] & (1 << 31)) >> 28);
			InputPtr += InputInc;
			OutputPtr[i] = (uint8)idx;
			NumOutputInstances += VVM_OUTPUT_ADVANCE_TABLE[idx];
		}
		if (NumLeftoverInstances > 0)
		{
			uint32 Index = 0;
			switch (NumLeftoverInstances)
			{
				case 1:	Index = ((InputPtr[0] & (1 << 31)) >> 31);                                                                           break;
				case 2:	Index = ((InputPtr[0] & (1 << 31)) >> 31) | ((InputPtr[1] & (1 << 31)) >> 30);                                       break;
				case 3:	Index = ((InputPtr[0] & (1 << 31)) >> 31) | ((InputPtr[1] & (1 << 31)) >> 30) | ((InputPtr[2] & (1 << 31)) >> 29);   break;
			}
			OutputPtr[NumFullLoops] = Index;
			NumOutputInstances += VVM_OUTPUT_ADVANCE_TABLE[Index];
		}
		NumOutputInstances >>= 2;
	}
	else
	{
		uint32 Index = 0;
		switch (BatchState->ChunkLocalData.NumInstancesThisChunk)
		{
			case 1:	Index = ((InputPtr[0] & (1 << 31)) >> 31);                                                                                                               break;
			case 2:	Index = ((InputPtr[0] & (1 << 31)) >> 31) | ((InputPtr[1] & (1 << 31)) >> 30);                                                                           break;
			case 3:	Index = ((InputPtr[0] & (1 << 31)) >> 31) | ((InputPtr[1] & (1 << 31)) >> 30) | ((InputPtr[2] & (1 << 31)) >> 29);                                       break;
			case 4: Index = ((InputPtr[0] & (1 << 31)) >> 31) | ((InputPtr[1] & (1 << 31)) >> 30) | ((InputPtr[2] & (1 << 31)) >> 29) | ((InputPtr[3] & (1 << 31)) >> 28);   break;
		}
		OutputPtr[0] = Index;
		NumOutputInstances += VVM_OUTPUT_ADVANCE_TABLE[Index] >> 2;
	}
	
	BatchState->ChunkLocalData.StartingOutputIdxPerDataSet[DataSetIdx] = ExecCtx->DataSets[DataSetIdx].InstanceOffset + FPlatformAtomics::InterlockedAdd(ExecCtx->VVMState->NumOutputPerDataSet + DataSetIdx, NumOutputInstances);
	BatchState->ChunkLocalData.NumOutputPerDataSet[DataSetIdx] += NumOutputInstances;

	return InsPtr + 4;
}

VM_FORCEINLINE const uint8 *VVM_exec_index(const bool CT_MultipleLoops, const uint8 *InsPtr, FVectorVMBatchState *BatchState, int NumLoops)
{
	if (CT_MultipleLoops)
	{
		uint8 *P0 = BatchState->RegPtrTable[((uint16 *)InsPtr)[0]];
		VectorRegister4i Val = VectorIntAdd(VectorIntSet1(BatchState->ChunkLocalData.StartInstanceThisChunk), VVM_m128iConst(ZeroOneTwoThree));
		VectorRegister4i Four = VectorIntSet1(4);
		uint8 *End = P0 + sizeof(FVecReg) * NumLoops;
		do
		{
			VectorIntStore(Val, P0);
			Val = VectorIntAdd(Val, Four);
			P0 += sizeof(FVecReg);
		} while (P0 < End);
	}
	else
	{
		VectorIntStore(VVM_m128iConst(ZeroOneTwoThree), BatchState->RegPtrTable[((uint16 *)InsPtr)[0]]);
	}
	return InsPtr + 3;
}

VM_FORCEINLINE const uint8 *VVM_exec_indexf(const bool CT_MultipleLoops, const uint8 *InsPtr, FVectorVMBatchState *BatchState, int NumLoops)
{
	if (CT_MultipleLoops)
	{
		uint8 *P0 = BatchState->RegPtrTable[((uint16 *)InsPtr)[0]];
		VectorRegister4i Val = VectorIntAdd(VectorIntSet1(BatchState->ChunkLocalData.StartInstanceThisChunk), VVM_m128iConst(ZeroOneTwoThree));
		VectorRegister4i Four = VectorIntSet1(4);
		uint8 *End = P0 + sizeof(FVecReg) * NumLoops;
		do
		{
			//this is faster than doing the work natively in floating point
			VectorStore(VectorIntToFloat(Val), (float *)P0);
			Val = VectorIntAdd(Val, Four);
			P0 += sizeof(FVecReg);
		} while (P0 < End);
	}
	else
	{
		VectorStore(VVM_m128Const(ZeroOneTwoThree), (float *)BatchState->RegPtrTable[((uint16 *)InsPtr)[0]]);
	}
	return InsPtr + 3;
}

VM_FORCEINLINE const uint8 *VVM_exec_index_addi(const bool CT_MultipleLoops, const uint8 *InsPtr, FVectorVMBatchState *BatchState, int NumLoops)
{
	if (CT_MultipleLoops)
	{
		uint8 *P0   = BatchState->RegPtrTable[((uint16 *)InsPtr)[0]];
		uint8 *P1   = BatchState->RegPtrTable[((uint16 *)InsPtr)[1]];
		uint32 Inc0 = (uint32)BatchState->RegIncTable[((uint16 *)InsPtr)[0]];
		VectorRegister4i Val = VectorIntAdd(VectorIntSet1(BatchState->ChunkLocalData.StartInstanceThisChunk), VVM_m128iConst(ZeroOneTwoThree));
		VectorRegister4i Four = VectorIntSet1(4);
		uint8 *End = P1 + sizeof(FVecReg) * NumLoops;
		do
		{
			VectorRegister4i R0 = VectorIntLoad(P0);
			P0 += Inc0;
			VectorIntStore(VectorIntAdd(Val, R0), P1);
			Val = VectorIntAdd(Val, Four);
			P1 += sizeof(FVecReg);
		} while (P1 < End);
	}
	else
	{
		VectorRegister4i R0 = VectorIntLoad(BatchState->RegPtrTable[((uint16 *)InsPtr)[0]]);
		VectorRegister4i Res = VectorIntAdd(R0, VVM_m128iConst(ZeroOneTwoThree));
		VectorIntStore(Res, BatchState->RegPtrTable[((uint16 *)InsPtr)[1]]);
	}
	return InsPtr + 5;
}

VM_FORCEINLINE VectorRegister4f VVM_nextRandom(FVectorVMBatchState *BatchState, VectorRegister4f a)
{
	return VectorMultiply(VectorSubtract(VectorRegister4f(VectorCastIntToFloat(VectorIntOr(VectorShiftRightImmLogical(VVMXorwowStep(BatchState), 9), VectorIntSet1(0x3F800000)))), VVM_m128Const(One)), a);
}

static VM_FORCEINLINE int32 VVM_SafeIntDivide(int32 Numerator, int32 Denominator)
{
	static constexpr int32 MinIntValue = std::numeric_limits<int32>::min();
	static constexpr int32 MaxIntValue = std::numeric_limits<int32>::max();

	if (Denominator == 0)
	{
		return 0;
	}
	else if ((Denominator == -1) && (Numerator == MinIntValue))
	{
		return MaxIntValue;
	}

	return Numerator / Denominator;
}

static VM_FORCEINLINE VectorRegister4i VVMIntDiv(VectorRegister4i v0, VectorRegister4i v1)
{
	const int32 *v0_4 = reinterpret_cast<const int32*>(&v0);
	const int32 *v1_4 = reinterpret_cast<const int32*>(&v1);

	FVVM_VUI4 res;
	res.i4[0] = VVM_SafeIntDivide(v0_4[0], v1_4[0]);
	res.i4[1] = VVM_SafeIntDivide(v0_4[1], v1_4[1]);
	res.i4[2] = VVM_SafeIntDivide(v0_4[2], v1_4[2]);
	res.i4[3] = VVM_SafeIntDivide(v0_4[3], v1_4[3]);
	
	return res.v;
}


static VM_FORCEINLINE const uint8 *VVM_random(const bool CT_MultipleLoops, const uint8 *InsPtr, FVectorVMBatchState *BatchState, FVectorVMExecContext *ExecCtx, int NumLoops)
{
	uint16* RegIndices = (uint16*)InsPtr;
	uint8 *P0   = BatchState->RegPtrTable[RegIndices[0]];
	uint8 *P1   = BatchState->RegPtrTable[RegIndices[1]];
	uint32 Inc0 = (uint32)BatchState->RegIncTable[RegIndices[0]];
	if (CT_MultipleLoops)
	{
		uint8 *End = P1 + sizeof(FVecReg) * NumLoops;
		do
		{
			VectorRegister4f R0 = VectorLoad((float *)P0);
			P0 += Inc0;
			VectorRegister4f Res = VVM_nextRandom(BatchState, R0);
			VectorStore(Res, (float *)P1);
			P1 += sizeof(FVecReg);
		} while (P1 < End);
	}
	else
	{
		VectorRegister4f R0 = VectorLoad((float *)P0);
		VectorRegister4f Res = VVM_nextRandom(BatchState, R0);
		VectorStore(Res, (float *)P1);
	}

	return InsPtr + 5;
}

static VM_FORCEINLINE const uint8 *VVM_randomi(const bool CT_MultipleLoops, const uint8 *InsPtr, FVectorVMBatchState *BatchState, FVectorVMExecContext *ExecCtx, int NumLoops)
{
	uint16* RegIndices = (uint16*)InsPtr;
	uint8 *P0   = BatchState->RegPtrTable[RegIndices[0]];
	uint8 *P1   = BatchState->RegPtrTable[RegIndices[1]];
	uint32 Inc0 = (uint32)BatchState->RegIncTable[RegIndices[0]];
	if (CT_MultipleLoops)
	{
		uint8 *End = P1 + sizeof(FVecReg) * NumLoops;
		do
		{
			VectorRegister4f R0 = VectorLoad((float *)P0);
			P0 += Inc0;
			VectorRegister4f Res = VVM_nextRandom(BatchState, R0);
			VectorIntStore(VectorFloatToInt(Res), P0);
			P1 += sizeof(FVecReg);
		} while (P1 < End);
	}
	else
	{
		VectorRegister4f R0 = VectorLoad((float *)P0);
		VectorRegister4f Res = VVM_nextRandom(BatchState, R0);
		VectorIntStore(VectorFloatToInt(Res), P0);
	}
	return InsPtr + 5;
}

static VM_FORCEINLINE const uint8 *VVM_random_add(const bool CT_MultipleLoops, const uint8 *InsPtr, FVectorVMBatchState *BatchState, FVectorVMExecContext *ExecCtx, int NumLoops)
{
	uint16* RegIndices = (uint16*)InsPtr;
	uint8 *P0   = BatchState->RegPtrTable[RegIndices[0]];
	uint8 *P1   = BatchState->RegPtrTable[RegIndices[1]];
	uint8 *P2   = BatchState->RegPtrTable[RegIndices[2]];
	uint32 Inc0 = (uint32)BatchState->RegIncTable[RegIndices[0]];
	uint32 Inc1 = (uint32)BatchState->RegIncTable[RegIndices[1]];

	if (CT_MultipleLoops)
	{
		uint8 *End = P2 + sizeof(FVecReg) * NumLoops;
		do
		{
			VectorRegister4f R0 = VectorLoad((float *)P0);
			VectorRegister4f R1 = VectorLoad((float *)P1);
			P0 += Inc0;
			P1 += Inc1;
			VectorRegister4f Res = VectorAdd(VVM_nextRandom(BatchState, R0), R1);
			VectorStore(Res, (float *)P2);
			P2 += sizeof(FVecReg);
		} while (P2 < End);
	}
	else
	{
		VectorRegister4f R0 = VectorLoad((float *)P0);
		VectorRegister4f R1 = VectorLoad((float *)P1);
		VectorRegister4f Res = VectorAdd(VVM_nextRandom(BatchState, R0), R1);
		VectorStore(Res, (float *)P2);
	}

	return InsPtr + 7;
}

static VM_FORCEINLINE const uint8 *VVM_random_2x(const bool CT_MultipleLoops, const uint8 *InsPtr, FVectorVMBatchState *BatchState, FVectorVMExecContext *ExecCtx, int NumLoops)
{
	uint16* RegIndices = (uint16*)InsPtr;
	uint8* P0 = BatchState->RegPtrTable[RegIndices[0]];
	uint8* P1 = BatchState->RegPtrTable[RegIndices[1]];
	uint8* P2 = BatchState->RegPtrTable[RegIndices[2]];
	uint32 Inc0 = (uint32)BatchState->RegIncTable[RegIndices[0]];
	if (CT_MultipleLoops)
	{
		uint8* End = P1 + sizeof(FVecReg) * NumLoops;
		do
		{
			VectorRegister4f R0 = VectorLoad((float*)P0);
			P0 += Inc0;
			VectorRegister4f Res0 = VVM_nextRandom(BatchState, R0);
			VectorRegister4f Res1 = VVM_nextRandom(BatchState, R0);
			VectorStore(Res0, (float*)P1);
			VectorStore(Res1, (float*)P2);
			P1 += sizeof(FVecReg);
			P2 += sizeof(FVecReg);
		} while (P1 < End);
	}
	else
	{
		VectorRegister4f R0 = VectorLoad((float*)P0);
		VectorRegister4f Res0 = VVM_nextRandom(BatchState, R0);
		VectorRegister4f Res1 = VVM_nextRandom(BatchState, R0);
		VectorStore(Res0, (float*)P1);
		VectorStore(Res1, (float*)P2);
	}

	return InsPtr + 7;
}

static VM_FORCEINLINE VectorRegister4f VVM_noise(VectorRegister4f a)
{
	return GlobalVectorConstants::FloatZero;
}

static VM_FORCEINLINE VectorRegister4f VVM_noise(VectorRegister4f x, VectorRegister4f y)
{
	return GlobalVectorConstants::FloatZero;
}

static VM_FORCEINLINE VectorRegister4f VVM_noise(VectorRegister4f x, VectorRegister4f y, VectorRegister4f z)
{
	return GlobalVectorConstants::FloatZero;
}

VM_FORCEINLINE const uint8 *VVM_half_to_float(const bool CT_MultipleLoops, const uint8 *InsPtr, FVectorVMBatchState *BatchState, int NumLoops)
{
	uint16* RegIndices = (uint16*)InsPtr;
	uint8 *P0   = BatchState->RegPtrTable[RegIndices[0]];
	uint8 *P1   = BatchState->RegPtrTable[RegIndices[1]];
	uint32 Inc0 = (uint32)BatchState->RegIncTable[RegIndices[0]] >> 1; //move the input 2 bytes instead of four
	uint8 *End  = P1 + sizeof(FVecReg) * NumLoops;
	do
	{
		FPlatformMath::VectorLoadHalf((float *)P1, (uint16 *)P0);
		P0 += Inc0;
		P1 += sizeof(FVecReg);
	} while (P1 < End);
	return InsPtr + 5;
}

static const uint8 *VVM_update_id(const bool CT_MultipleLoops, const uint8 *InsPtr, FVectorVMBatchState *BatchState, FVectorVMExecContext *ExecCtx, int NumLoops)
{
	uint16* RegIndices = (uint16*)InsPtr;
	int32 *R1             = (int32 *)BatchState->RegPtrTable[RegIndices[0]];
	int32 *R2             = (int32 *)BatchState->RegPtrTable[RegIndices[1]];
	uint8 DataSetIdx      = InsPtr[4];
	FDataSetMeta *DataSet = &ExecCtx->DataSets[DataSetIdx];
	check(DataSetIdx < (uint32)ExecCtx->DataSets.Num());
	check(DataSet->IDTable);
	check(DataSet->IDTable->Num() >= DataSet->InstanceOffset + BatchState->ChunkLocalData.StartInstanceThisChunk + BatchState->ChunkLocalData.NumInstancesThisChunk);
	int NumOutputInstances = BatchState->ChunkLocalData.NumOutputPerDataSet[DataSetIdx];
	int NumFreed           = BatchState->ChunkLocalData.NumInstancesThisChunk - BatchState->ChunkLocalData.NumOutputPerDataSet[DataSetIdx];
				
	//compute this chunk's MaxID
	int MaxID = -1;
	if (NumOutputInstances > 4)
	{
		int NumOutput4 = (int)(((((uint32)NumOutputInstances + 3U) & ~3U) - 1) >> 2);
		VectorRegister4i Max4 = VectorIntSet1(-1);
		for (int i = 0; i < NumOutput4; ++i)
		{
			VectorRegister4i R4 = VectorIntLoad(R1 + (uint64)((uint64)i << 2ULL));
			Max4 = VectorIntMax(Max4, R4);
		}
		VectorRegister4i Last4 = VectorIntLoad(R1 + NumOutputInstances - 4);
		Max4 = VectorIntMax(Last4, Max4);
		int M4[4];
		VectorIntStore(Max4, M4);
		int m0 = M4[0] > M4[1] ? M4[0] : M4[1];
		int m1 = M4[2] > M4[3] ? M4[2] : M4[3];
		int m = m0 > m1 ? m0 : m1;
		if (m > MaxID)
		{
			MaxID = m;
		}
	}
	else
	{
		for (int i = 0; i < NumOutputInstances; ++i)
		{
			if (R1[i] > MaxID)
			{
				MaxID = R1[i];
			}
		}
	}
	int StartNumFreed        = FPlatformAtomics::InterlockedAdd((volatile int32 *)DataSet->NumFreeIDs, NumFreed);
	int NumFullLoops         = BatchState->ChunkLocalData.NumInstancesThisChunk >> 2;
	int NumLeftoverInstances = BatchState->ChunkLocalData.NumInstancesThisChunk - (NumFullLoops << 2);
	uint8 *OutputMaskIdx     = BatchState->ChunkLocalData.OutputMaskIdx[DataSetIdx];

	int *IDTable             = DataSet->IDTable->GetData();
	int *FreeTable           = DataSet->FreeIDTable->GetData() + StartNumFreed;

	int NumInstancesCounted  = 0;
	int NumInstancesFreed    = 0;
	NumInstancesCounted = 0;
	{
		for (int i = 0; i < NumFullLoops; ++i)
		{
			int LoopCount         = i << 2;
			uint8 Idx             = OutputMaskIdx[i];
			int InsCnt            = VVM_OUTPUT_ADVANCE_TABLE[Idx] >> 2;
			int StartingOutputIdx = BatchState->ChunkLocalData.StartingOutputIdxPerDataSet[DataSetIdx] + NumInstancesCounted;
			int StartingNumFreed  = NumInstancesFreed;

			NumInstancesCounted += InsCnt;
			NumInstancesFreed += 4 - InsCnt;

			switch (Idx)
			{
				case 0:		FreeTable[StartingNumFreed + 0] = R1[LoopCount + 0];
							FreeTable[StartingNumFreed + 1] = R1[LoopCount + 1];
							FreeTable[StartingNumFreed + 2] = R1[LoopCount + 2];
							FreeTable[StartingNumFreed + 3] = R1[LoopCount + 3];
							break;
				case 1:		IDTable[R1[LoopCount + 0]]      = StartingOutputIdx + 0; 
							FreeTable[StartingNumFreed + 0] = R1[LoopCount + 1];
							FreeTable[StartingNumFreed + 1] = R1[LoopCount + 2];
							FreeTable[StartingNumFreed + 2] = R1[LoopCount + 3];
							break;
				case 2:		IDTable[R1[LoopCount + 1]]      = StartingOutputIdx + 0; 
							FreeTable[StartingNumFreed + 0] = R1[LoopCount + 0];
							FreeTable[StartingNumFreed + 1] = R1[LoopCount + 2];
							FreeTable[StartingNumFreed + 2] = R1[LoopCount + 3];
							break;
				case 3:		IDTable[R1[LoopCount + 0]]      = StartingOutputIdx + 0;
							IDTable[R1[LoopCount + 1]]      = StartingOutputIdx + 1; 
							FreeTable[StartingNumFreed + 0] = R1[LoopCount + 2];
							FreeTable[StartingNumFreed + 1] = R1[LoopCount + 3];
							break;
				case 4:		IDTable[R1[LoopCount + 2]]      = StartingOutputIdx + 0; 
							FreeTable[StartingNumFreed + 0] = R1[LoopCount + 0];
							FreeTable[StartingNumFreed + 1] = R1[LoopCount + 1];
							FreeTable[StartingNumFreed + 2] = R1[LoopCount + 3];
							break;
				case 5:		IDTable[R1[LoopCount + 0]]      = StartingOutputIdx + 0;
							IDTable[R1[LoopCount + 2]]      = StartingOutputIdx + 1; 
							FreeTable[StartingNumFreed + 0] = R1[LoopCount + 1];
							FreeTable[StartingNumFreed + 1] = R1[LoopCount + 3];
							break;
				case 6:		IDTable[R1[LoopCount + 1]]      = StartingOutputIdx + 0;
							IDTable[R1[LoopCount + 2]]      = StartingOutputIdx + 1;
							FreeTable[StartingNumFreed + 0] = R1[LoopCount + 0];
							FreeTable[StartingNumFreed + 1] = R1[LoopCount + 3];
							break;
				case 7:		IDTable[R1[LoopCount + 0]]      = StartingOutputIdx + 0;
							IDTable[R1[LoopCount + 1]]      = StartingOutputIdx + 1;
							IDTable[R1[LoopCount + 2]]      = StartingOutputIdx + 2; 
							FreeTable[StartingNumFreed + 0] = R1[LoopCount + 3];
							break;
				case 8:		IDTable[R1[LoopCount + 3]]      = StartingOutputIdx + 0; 
							FreeTable[StartingNumFreed + 0] = R1[LoopCount + 0];
							FreeTable[StartingNumFreed + 1] = R1[LoopCount + 1];
							FreeTable[StartingNumFreed + 2] = R1[LoopCount + 2];
							break;
				case 9:		IDTable[R1[LoopCount + 0]]      = StartingOutputIdx + 0;
							IDTable[R1[LoopCount + 3]]      = StartingOutputIdx + 1; 
							FreeTable[StartingNumFreed + 0] = R1[LoopCount + 1];
							FreeTable[StartingNumFreed + 1] = R1[LoopCount + 2];
							break;
				case 10:	IDTable[R1[LoopCount + 1]]      = StartingOutputIdx + 0;
							IDTable[R1[LoopCount + 3]]      = StartingOutputIdx + 1;
							FreeTable[StartingNumFreed + 0] = R1[LoopCount + 0];
							FreeTable[StartingNumFreed + 1] = R1[LoopCount + 2];
							break;
				case 11:	IDTable[R1[LoopCount + 0]]      = StartingOutputIdx + 0;
							IDTable[R1[LoopCount + 1]]      = StartingOutputIdx + 1;
							IDTable[R1[LoopCount + 3]]      = StartingOutputIdx + 2; 
							FreeTable[StartingNumFreed + 0] = R1[LoopCount + 2];
							break;
				case 12:	IDTable[R1[LoopCount + 2]]      = StartingOutputIdx + 0;
							IDTable[R1[LoopCount + 3]]      = StartingOutputIdx + 1; 
							FreeTable[StartingNumFreed + 0] = R1[LoopCount + 0];
							FreeTable[StartingNumFreed + 1] = R1[LoopCount + 1];
							break;
				case 13:	IDTable[R1[LoopCount + 0]]      = StartingOutputIdx + 0;
							IDTable[R1[LoopCount + 2]]      = StartingOutputIdx + 1;
							IDTable[R1[LoopCount + 3]]      = StartingOutputIdx + 2;
							FreeTable[StartingNumFreed + 0] = R1[LoopCount + 1];
							break;
				case 14:	IDTable[R1[LoopCount + 1]]      = StartingOutputIdx + 0;
							IDTable[R1[LoopCount + 2]]      = StartingOutputIdx + 1;
							IDTable[R1[LoopCount + 3]]      = StartingOutputIdx + 2;
							FreeTable[StartingNumFreed + 0] = R1[LoopCount + 0];
							break;
				case 15:	IDTable[R1[LoopCount + 0]]      = StartingOutputIdx + 0;
							IDTable[R1[LoopCount + 1]]      = StartingOutputIdx + 1;
							IDTable[R1[LoopCount + 2]]      = StartingOutputIdx + 2;
							IDTable[R1[LoopCount + 3]]      = StartingOutputIdx + 3;
							break;
			}
					
		}

		//if (NumLeftoverInstances != 0) {
		{
			int StartingOutputIdx = BatchState->ChunkLocalData.StartingOutputIdxPerDataSet[DataSetIdx] + NumInstancesCounted;
			int LoopCount         = NumFullLoops << 2;
			uint8 Idx             = OutputMaskIdx[NumFullLoops];
			switch (NumLeftoverInstances)
			{
				case 0: break;
				case 1:
					if (Idx & 1)
					{
						IDTable[R1[LoopCount]] = StartingOutputIdx;
						++NumInstancesCounted;
					}
					else
					{
						FreeTable[NumInstancesFreed] = R1[LoopCount];
						++NumInstancesFreed;
					}
					break;
				case 2:
					switch (Idx & 3)
					{
						case 0:		FreeTable[NumInstancesFreed + 0] = R1[LoopCount + 0];
									FreeTable[NumInstancesFreed + 1] = R1[LoopCount + 1];
									NumInstancesFreed += 2;
									break;
						case 1:		IDTable[R1[LoopCount + 0]]       = StartingOutputIdx + 0; 
									FreeTable[NumInstancesFreed + 0] = R1[LoopCount + 1];
									++NumInstancesCounted;
									++NumInstancesFreed;
									break; 
						case 2:		IDTable[R1[LoopCount + 1]]       = StartingOutputIdx + 0; 
									FreeTable[NumInstancesFreed + 0] = R1[LoopCount + 0];
									++NumInstancesCounted;
									++NumInstancesFreed;
									break;
						case 3:		IDTable[R1[LoopCount + 0]] = StartingOutputIdx + 0;
									IDTable[R1[LoopCount + 1]] = StartingOutputIdx + 1;
									NumInstancesCounted += 2;
									break;
					}
					break;
				case 3:
					switch (Idx & 7)
					{
						case 0:		FreeTable[NumInstancesFreed + 0] = R1[LoopCount + 0];
									FreeTable[NumInstancesFreed + 1] = R1[LoopCount + 1];
									FreeTable[NumInstancesFreed + 2] = R1[LoopCount + 2];
									NumInstancesFreed += 3;
									break;
						case 1:		IDTable[R1[LoopCount + 0]]       = StartingOutputIdx + 0; 
									FreeTable[NumInstancesFreed + 0] = R1[LoopCount + 1];
									FreeTable[NumInstancesFreed + 1] = R1[LoopCount + 2];
									++NumInstancesCounted;
									NumInstancesFreed += 2;
									break;
						case 2:		IDTable[R1[LoopCount + 1]]       = StartingOutputIdx + 0; 
									FreeTable[NumInstancesFreed + 0] = R1[LoopCount + 0];
									FreeTable[NumInstancesFreed + 1] = R1[LoopCount + 2];
									++NumInstancesCounted;
									NumInstancesFreed += 2;
									break;
						case 3:		IDTable[R1[LoopCount + 0]]       = StartingOutputIdx + 0;
									IDTable[R1[LoopCount + 1]]       = StartingOutputIdx + 1;
									FreeTable[NumInstancesFreed + 0] = R1[LoopCount + 2];
									NumInstancesCounted += 2;
									++NumInstancesFreed;
									break;
						case 4:		IDTable[R1[LoopCount + 2]]       = StartingOutputIdx + 0; 
									FreeTable[NumInstancesFreed + 0] = R1[LoopCount + 0];
									FreeTable[NumInstancesFreed + 1] = R1[LoopCount + 1];
									++NumInstancesCounted;
									NumInstancesFreed += 2;
									break;
						case 5:		IDTable[R1[LoopCount + 0]]       = StartingOutputIdx + 0;
									IDTable[R1[LoopCount + 2]]       = StartingOutputIdx + 1;
									FreeTable[NumInstancesFreed + 0] = R1[LoopCount + 1];
									NumInstancesCounted += 2;
									++NumInstancesFreed;
									break;
						case 6:		IDTable[R1[LoopCount + 1]]       = StartingOutputIdx + 0;
									IDTable[R1[LoopCount + 2]]       = StartingOutputIdx + 1;
									FreeTable[NumInstancesFreed + 0] = R1[LoopCount + 0];
									NumInstancesCounted += 2;
									++NumInstancesFreed;
									break;
						case 7:		IDTable[R1[LoopCount + 0]] = StartingOutputIdx + 0;
									IDTable[R1[LoopCount + 1]] = StartingOutputIdx + 1;
									IDTable[R1[LoopCount + 2]] = StartingOutputIdx + 2;
									NumInstancesCounted += 3;
									break;
					}
					break;
			}
		}
	}

	//Set the DataSet's MaxID if this chunk's MaxID is bigger
	if (MaxID != -1)
	{
		int SanityCount = 0;
		do
		{
			int OldMaxID = FPlatformAtomics::AtomicRead(DataSet->MaxUsedID);
			if (MaxID <= OldMaxID)
			{
				break;
			}
			int NewMaxID = FPlatformAtomics::InterlockedCompareExchange((volatile int32 *)DataSet->MaxUsedID, MaxID, OldMaxID);
			if (NewMaxID == OldMaxID)
			{
				break;
			}
		} while (SanityCount++ < (1 << 30));
		VVMDebugBreakIf(SanityCount > (1 << 30) - 1);
	}
	return InsPtr + 6;
}

static const uint8 *VVM_acquire_id(const bool CT_MultipleLoops, const uint8 *InsPtr, FVectorVMBatchState *BatchState, FVectorVMExecContext *ExecCtx, int NumLoops)
{
	uint8 DataSetIdx = InsPtr[4];
	check(DataSetIdx < (uint32)ExecCtx->DataSets.Num());
	FDataSetMeta *DataSet = &ExecCtx->DataSets[DataSetIdx];

	{ //1. Get the free IDs into the temp register
		int SanityCount = 0;
		do
		{
			int OldNumFreeIDs = FPlatformAtomics::AtomicRead(DataSet->NumFreeIDs);
			check(OldNumFreeIDs >= BatchState->ChunkLocalData.NumInstancesThisChunk);
			int *OutPtr = (int *)BatchState->RegPtrTable[((uint16 *)InsPtr)[0]];
			int *InPtr  = DataSet->FreeIDTable->GetData() + OldNumFreeIDs - BatchState->ChunkLocalData.NumInstancesThisChunk;
			for (int i = 0; i < BatchState->ChunkLocalData.NumInstancesThisChunk; ++i)
			{
				OutPtr[i] = InPtr[BatchState->ChunkLocalData.NumInstancesThisChunk - i - 1];
			}
			int NewNumFreeIDs = FPlatformAtomics::InterlockedCompareExchange((volatile int32 *)DataSet->NumFreeIDs, OldNumFreeIDs - BatchState->ChunkLocalData.NumInstancesThisChunk, OldNumFreeIDs);
			if (NewNumFreeIDs == OldNumFreeIDs)
			{
				break;
			}
		} while (SanityCount++ < (1 << 30));
		VVMDebugBreakIf(SanityCount >= (1 << 30) - 1);
	}
	{ //2. append the IDs we acquired in step 1 to the end of the free table array, representing spawned IDs
		//FreeID table is write-only as far as this invocation of the VM is concerned, so the interlocked add w/o filling
		//in the data is fine
		int StartNumSpawned = FPlatformAtomics::InterlockedAdd(DataSet->NumSpawnedIDs, BatchState->ChunkLocalData.NumInstancesThisChunk) + BatchState->ChunkLocalData.NumInstancesThisChunk;
		check(StartNumSpawned <= DataSet->FreeIDTable->Max());
		VVMMemCpy(DataSet->FreeIDTable->GetData() + DataSet->FreeIDTable->Max() - StartNumSpawned, BatchState->RegPtrTable[((uint16 *)InsPtr)[0]], sizeof(int32) * BatchState->ChunkLocalData.NumInstancesThisChunk);
	}
	//3. set the tag
	VVMMemSet32(BatchState->RegPtrTable[((uint16 *)InsPtr)[1]], DataSet->IDAcquireTag, BatchState->ChunkLocalData.NumInstancesThisChunk);
	return InsPtr + 6;
}

const uint8 *VVM_external_func_call(const bool CT_MultipleLoops, const uint8 *InsPtr, FVectorVMBatchState *BatchState, FVectorVMExecContext *ExecCtx, int NumLoops)
{
	int FnIdx = (int)*(uint16 *)(InsPtr);
	FVectorVMExtFunctionData *ExtFnData = ExecCtx->VVMState->ExtFunctionTable + FnIdx;

	{
		check(*InsPtr < ExecCtx->VVMState->NumExtFunctions);
		check((uint32)(ExtFnData->NumInputs + ExtFnData->NumOutputs) <= ExecCtx->VVMState->MaxExtFnRegisters);
		const uint16 *RegIndices      = ((uint16 *)InsPtr) + 1;
		uint32 DummyRegCount = 0;
		for (int i = 0; i < ExtFnData->NumInputs + ExtFnData->NumOutputs; ++i)
		{
			if (RegIndices[i] != 0xFFFF)
			{
				BatchState->ChunkLocalData.ExtFnDecodedReg.RegData[i] = (uint32 *)BatchState->RegPtrTable[RegIndices[i]];
				BatchState->ChunkLocalData.ExtFnDecodedReg.RegInc[i]  = BatchState->RegIncTable[RegIndices[i]] >> 4; //external functions increment by 1 32 bit value at a time
			}
			else
			{
				BatchState->ChunkLocalData.ExtFnDecodedReg.RegData[i] = (uint32 *)(BatchState->ChunkLocalData.ExtFnDecodedReg.DummyRegs + DummyRegCount++);
				BatchState->ChunkLocalData.ExtFnDecodedReg.RegInc[i]  = 0;
			}
		}
		check(DummyRegCount <= ExecCtx->VVMState->NumDummyRegsRequired);

		FVectorVMExternalFunctionContext ExtFnCtx;

		ExtFnCtx.RegisterData             = BatchState->ChunkLocalData.ExtFnDecodedReg.RegData;
		ExtFnCtx.RegInc                   = BatchState->ChunkLocalData.ExtFnDecodedReg.RegInc;

		ExtFnCtx.RegReadCount             = 0;
		ExtFnCtx.NumRegisters             = ExtFnData->NumInputs + ExtFnData->NumOutputs;

		ExtFnCtx.StartInstance            = BatchState->ChunkLocalData.StartInstanceThisChunk;
		ExtFnCtx.NumInstances             = BatchState->ChunkLocalData.NumInstancesThisChunk;
		ExtFnCtx.NumLoops                 = NumLoops;
		ExtFnCtx.PerInstanceFnInstanceIdx = 0;

		ExtFnCtx.UserPtrTable             = ExecCtx->UserPtrTable.GetData();
		ExtFnCtx.NumUserPtrs              = ExecCtx->UserPtrTable.Num();
		ExtFnCtx.RandStream               = &BatchState->RandStream;

		ExtFnCtx.RandCounters             = &BatchState->ChunkLocalData.RandCounters;
		ExtFnCtx.DataSets                 = ExecCtx->DataSets;
		ExtFnData->Function->Execute(ExtFnCtx);
	}
	return InsPtr + 3 + 2 * (ExtFnData->NumInputs + ExtFnData->NumOutputs);
}

static VM_FORCEINLINE const uint8 *VVM_sincos(const bool CT_MultipleLoops, const uint8 *InsPtr, FVectorVMBatchState *BatchState, int NumLoops)
{
	uint16 *RegIndices = (uint16 *)InsPtr;
	if (CT_MultipleLoops)
	{
		uint8 *P0   = BatchState->RegPtrTable[RegIndices[0]];
		uint8 *P1   = BatchState->RegPtrTable[RegIndices[1]];
		uint8 *P2   = BatchState->RegPtrTable[RegIndices[2]];
		uint32 Inc0 = (uint32)BatchState->RegIncTable[RegIndices[0]];
		uint8 *End = P1 + sizeof(FVecReg) * NumLoops;
		do
		{
			VectorRegister4f R0 = VectorLoad((float *)P0);
			P0 += Inc0;
			VectorSinCos((VectorRegister4f *)P1, (VectorRegister4f *)P2, &R0);
			P1 += sizeof(FVecReg);
			P2 += sizeof(FVecReg);
		} while (P1 < End);
	}
	else
	{
		VectorRegister4f r0 = VectorLoad((float *)BatchState->RegPtrTable[RegIndices[0]]);
		VectorSinCos((VectorRegister4f *)BatchState->RegPtrTable[RegIndices[1]], (VectorRegister4f *)BatchState->RegPtrTable[RegIndices[2]], &r0);
	}
	return InsPtr + 7;
}

//zF16 - stb_image_resize

#define VVM_NULL_FN_ARGS const bool CT_MultipleLoops, const uint8 *InsPtr, FVectorVMBatchState *BatchState, FVectorVMExecContext *ExecCtx, int NumLoops

typedef const uint8 *    (VVMFn_null)(VVM_NULL_FN_ARGS);
typedef VectorRegister4f (VVMFn_1f)  (FVectorVMBatchState *, VectorRegister4f a);
typedef VectorRegister4f (VVMFn_2f)  (FVectorVMBatchState *, VectorRegister4f a, VectorRegister4f b);
typedef VectorRegister4f (VVMFn_3f)  (FVectorVMBatchState *, VectorRegister4f a, VectorRegister4f b, VectorRegister4f c);
typedef VectorRegister4f (VVMFn_4f)  (FVectorVMBatchState *, VectorRegister4f a, VectorRegister4f b, VectorRegister4f c, VectorRegister4f d);
typedef VectorRegister4f (VVMFn_5f)  (FVectorVMBatchState *, VectorRegister4f a, VectorRegister4f b, VectorRegister4f c, VectorRegister4f d, VectorRegister4f e);
typedef VectorRegister4i (VVMFn_1i)  (FVectorVMBatchState *, VectorRegister4i a);
typedef VectorRegister4i (VVMFn_2i)  (FVectorVMBatchState *, VectorRegister4i a, VectorRegister4i b);
typedef VectorRegister4i (VVMFn_3i)  (FVectorVMBatchState *, VectorRegister4i a, VectorRegister4i b, VectorRegister4i c);

//Dispatch functions: these rely on VM_FORCEINLINE to actually force the function to be inlined
VM_FORCEINLINE const uint8 *VVM_Dispatch_execFn0null_0null(const bool CT_MultipleLoops, const uint8 *InsPtr, FVectorVMBatchState *BatchState, FVectorVMExecContext *ExecCtx, VVMFn_null fn, int NumLoops)
{
	return fn(CT_MultipleLoops, InsPtr, BatchState, ExecCtx, NumLoops);
}

VM_FORCEINLINE const uint8 *VVM_Dispatch_execFn1null_0null(const bool CT_MultipleLoops, const uint8 *InsPtr, FVectorVMBatchState *BatchState, FVectorVMExecContext *ExecCtx, VVMFn_null fn, int NumLoops)
{
	return fn(CT_MultipleLoops, InsPtr, BatchState, ExecCtx, NumLoops);
}

VM_FORCEINLINE const uint8 *VVM_Dispatch_execFn1null_1null(const bool CT_MultipleLoops, const uint8 *InsPtr, FVectorVMBatchState *BatchState, FVectorVMExecContext *ExecCtx, VVMFn_null fn, int NumLoops)
{
	return fn(CT_MultipleLoops, InsPtr, BatchState, ExecCtx, NumLoops);
}

VM_FORCEINLINE const uint8 *VVM_Dispatch_execFn1null_2null(const bool CT_MultipleLoops, const uint8 *InsPtr, FVectorVMBatchState *BatchState, FVectorVMExecContext *ExecCtx, VVMFn_null fn, int NumLoops)
{
	return fn(CT_MultipleLoops, InsPtr, BatchState, ExecCtx, NumLoops);
}

VM_FORCEINLINE const uint8 *VVM_Dispatch_execFn2null_1null(const bool CT_MultipleLoops, const uint8 *InsPtr, FVectorVMBatchState *BatchState, FVectorVMExecContext *ExecCtx, VVMFn_null fn, int NumLoops)
{
	return fn(CT_MultipleLoops, InsPtr, BatchState, ExecCtx, NumLoops);
}

VM_FORCEINLINE const uint8 *VVM_Dispatch_execFn1f_1f(const bool CT_MultipleLoops, const uint8 *InsPtr, FVectorVMBatchState *BatchState, FVectorVMExecContext *ExecCtx, VVMFn_1f fn, int NumLoops)
{
	uint16 *RegIndices = (uint16 *)InsPtr;
	uint8 *P0 = BatchState->RegPtrTable[RegIndices[0]];
	uint8 *P1 = BatchState->RegPtrTable[RegIndices[1]];
	if (CT_MultipleLoops)
	{
		uint32 Inc0 = (uint32)BatchState->RegIncTable[RegIndices[0]];
		uint8 *End  = P1 + sizeof(FVecReg) * NumLoops;
		do
		{
			VectorRegister4f R0 = VectorLoad((float *)P0);
			P0 += Inc0;
			VectorRegister4f Res = fn(BatchState, R0);
			VectorStore(Res, (float *)P1);
			P1 += sizeof(FVecReg);
		} while (P1 < End);
	}
	else
	{
		VectorRegister4f R0 = VectorLoad((float *)P0);
		VectorRegister4f Res = fn(BatchState, R0);
		VectorStore(Res, (float *)P1);
	}
	return InsPtr + 5;
}

VM_FORCEINLINE const uint8 *VVM_Dispatch_execFn2f_1f(const bool CT_MultipleLoops, const uint8 *InsPtr, FVectorVMBatchState *BatchState, FVectorVMExecContext *ExecCtx, VVMFn_2f fn, int NumLoops)
{
	uint16 *RegIndices = (uint16 *)InsPtr;
	uint8 *P0 = BatchState->RegPtrTable[RegIndices[0]];
	uint8 *P1 = BatchState->RegPtrTable[RegIndices[1]];
	uint8 *P2 = BatchState->RegPtrTable[RegIndices[2]];
	if (CT_MultipleLoops)
	{
		uint32 Inc0 = (uint32)BatchState->RegIncTable[RegIndices[0]];
		uint32 Inc1 = (uint32)BatchState->RegIncTable[RegIndices[1]];
		uint8 *End  = P2 + sizeof(FVecReg) * NumLoops;
		do
		{
			VectorRegister4f R0 = VectorLoad((float *)P0);
			VectorRegister4f R1 = VectorLoad((float *)P1);
			P0 += Inc0;
			P1 += Inc1;
			VectorRegister4f Res = fn(BatchState, R0, R1);
			VectorStore(Res, (float *)P2);
			P2 += sizeof(FVecReg);
		} while (P2 < End);
	}
	else
	{
		VectorRegister4f R0 = VectorLoad((float *)P0);
		VectorRegister4f R1 = VectorLoad((float *)P1);
		VectorRegister4f Res = fn(BatchState, R0, R1);
		VectorStore(Res, (float *)P2);
	}
	return InsPtr + 7;
}

VM_FORCEINLINE const uint8 *VVM_Dispatch_execFn3f_1f(const bool CT_MultipleLoops, const uint8 *InsPtr, FVectorVMBatchState *BatchState, FVectorVMExecContext *ExecCtx, VVMFn_3f fn, int NumLoops)
{
	uint16 *RegIndices = (uint16 *)InsPtr;
	uint8 *P0 = BatchState->RegPtrTable[RegIndices[0]];
	uint8 *P1 = BatchState->RegPtrTable[RegIndices[1]];
	uint8 *P2 = BatchState->RegPtrTable[RegIndices[2]];
	uint8 *P3 = BatchState->RegPtrTable[RegIndices[3]];
	if (CT_MultipleLoops)
	{
		uint32 Inc0 = (uint32)BatchState->RegIncTable[RegIndices[0]];
		uint32 Inc1 = (uint32)BatchState->RegIncTable[RegIndices[1]];
		uint32 Inc2 = (uint32)BatchState->RegIncTable[RegIndices[2]];
		uint8 *End  = P3 + sizeof(FVecReg) * NumLoops;
		do
		{
			VectorRegister4f R0 = VectorLoad((float *)P0);
			VectorRegister4f R1 = VectorLoad((float *)P1);
			VectorRegister4f R2 = VectorLoad((float *)P2);
			P0 += Inc0;
			P1 += Inc1;
			P2 += Inc2;
			VectorRegister4f Res = fn(BatchState, R0, R1, R2);
			VectorStore(Res, (float *)P3);
			P3 += sizeof(FVecReg);
		} while (P3 < End);
	}
	else
	{
		VectorRegister4f R0 = VectorLoad((float *)P0);
		VectorRegister4f R1 = VectorLoad((float *)P1);
		VectorRegister4f R2 = VectorLoad((float *)P2);
		VectorRegister4f Res = fn(BatchState, R0, R1, R2);
		VectorStore(Res, (float *)P3);
	}
	return InsPtr + 9;
}

VM_FORCEINLINE const uint8 *VVM_Dispatch_execFn4f_1f(const bool CT_MultipleLoops, const uint8 *InsPtr, FVectorVMBatchState *BatchState, FVectorVMExecContext *ExecCtx, VVMFn_4f fn, int NumLoops)
{
	uint16 *RegIndices = (uint16 *)InsPtr;
	uint8 *P0 = BatchState->RegPtrTable[RegIndices[0]];
	uint8 *P1 = BatchState->RegPtrTable[RegIndices[1]];
	uint8 *P2 = BatchState->RegPtrTable[RegIndices[2]];
	uint8 *P3 = BatchState->RegPtrTable[RegIndices[3]];
	uint8 *P4 = BatchState->RegPtrTable[RegIndices[4]];
	if (CT_MultipleLoops)
	{
		uint32 Inc0 = (uint32)BatchState->RegIncTable[RegIndices[0]];
		uint32 Inc1 = (uint32)BatchState->RegIncTable[RegIndices[1]];
		uint32 Inc2 = (uint32)BatchState->RegIncTable[RegIndices[2]];
		uint32 Inc3 = (uint32)BatchState->RegIncTable[RegIndices[3]];
		uint8 *End  = P4 + sizeof(FVecReg) * NumLoops;
		do
		{
			VectorRegister4f R0 = VectorLoad((float *)P0);
			VectorRegister4f R1 = VectorLoad((float *)P1);
			VectorRegister4f R2 = VectorLoad((float *)P2);
			VectorRegister4f R3 = VectorLoad((float *)P3);
			P0 += Inc0;
			P1 += Inc1;
			P2 += Inc2;
			P3 += Inc3;
			VectorRegister4f Res = fn(BatchState, R0, R1, R2, R3);
			VectorStore(Res, (float *)P4);
			P4 += sizeof(FVecReg);
		} while (P4 < End);
	}
	else
	{
		VectorRegister4f R0 = VectorLoad((float *)P0);
		VectorRegister4f R1 = VectorLoad((float *)P1);
		VectorRegister4f R2 = VectorLoad((float *)P2);
		VectorRegister4f R3 = VectorLoad((float *)P3);
		VectorRegister4f Res = fn(BatchState, R0, R1, R2, R3);
		VectorStore(Res, (float *)P4);
	}
	return InsPtr + 11;
}

VM_FORCEINLINE const uint8 *VVM_Dispatch_execFn5f_1f(const bool CT_MultipleLoops, const uint8 *InsPtr, FVectorVMBatchState *BatchState, FVectorVMExecContext *ExecCtx, VVMFn_5f fn, int NumLoops)
{
	uint16 *RegIndices = (uint16 *)InsPtr;
	uint8 *P0 = BatchState->RegPtrTable[RegIndices[0]];
	uint8 *P1 = BatchState->RegPtrTable[RegIndices[1]];
	uint8 *P2 = BatchState->RegPtrTable[RegIndices[2]];
	uint8 *P3 = BatchState->RegPtrTable[RegIndices[3]];
	uint8 *P4 = BatchState->RegPtrTable[RegIndices[4]];
	uint8 *P5 = BatchState->RegPtrTable[RegIndices[5]];
	if (CT_MultipleLoops)
	{
		uint32 Inc0 = (uint32)BatchState->RegIncTable[RegIndices[0]];
		uint32 Inc1 = (uint32)BatchState->RegIncTable[RegIndices[1]];
		uint32 Inc2 = (uint32)BatchState->RegIncTable[RegIndices[2]];
		uint32 Inc3 = (uint32)BatchState->RegIncTable[RegIndices[3]];
		uint32 Inc4 = (uint32)BatchState->RegIncTable[RegIndices[4]];
		uint8 *End  = P5 + sizeof(FVecReg) * NumLoops;
		do
		{
			VectorRegister4f R0 = VectorLoad((float *)P0);
			VectorRegister4f R1 = VectorLoad((float *)P1);
			VectorRegister4f R2 = VectorLoad((float *)P2);
			VectorRegister4f R3 = VectorLoad((float *)P3);
			VectorRegister4f R4 = VectorLoad((float *)P4);
			P0 += Inc0;
			P1 += Inc1;
			P2 += Inc2;
			P3 += Inc3;
			P4 += Inc4;
			VectorRegister4f Res = fn(BatchState, R0, R1, R2, R3, R4);
			VectorStore(Res, (float *)P5);
			P5 += sizeof(FVecReg);
		} while (P5 < End);
	}
	else
	{
		VectorRegister4f R0 = VectorLoad((float *)P0);
		VectorRegister4f R1 = VectorLoad((float *)P1);
		VectorRegister4f R2 = VectorLoad((float *)P2);
		VectorRegister4f R3 = VectorLoad((float *)P3);
		VectorRegister4f R4 = VectorLoad((float *)P4);
		VectorRegister4f Res = fn(BatchState, R0, R1, R2, R3, R4);
		VectorStore(Res, (float *)P5);
	}
	return InsPtr + 13;
}

VM_FORCEINLINE const uint8 *VVM_Dispatch_execFn2f_2f(const bool CT_MultipleLoops, const uint8 *InsPtr, FVectorVMBatchState *BatchState, FVectorVMExecContext *ExecCtx, VVMFn_2f fn, int NumLoops)
{
	uint16 *RegIndices = (uint16 *)InsPtr;
	uint8 *P0 = BatchState->RegPtrTable[RegIndices[0]];
	uint8 *P1 = BatchState->RegPtrTable[RegIndices[1]];
	uint8 *P2 = BatchState->RegPtrTable[RegIndices[2]];
	uint8 *P3 = BatchState->RegPtrTable[RegIndices[3]];
	if (CT_MultipleLoops)
	{
		uint32 Inc0    = (uint32)BatchState->RegIncTable[RegIndices[0]];
		uint32 Inc1    = (uint32)BatchState->RegIncTable[RegIndices[1]];
		ptrdiff_t P23d = P3 - P2;
		uint8 *End     = P3 + sizeof(FVecReg) * NumLoops;
		do
		{
			VectorRegister4f R0 = VectorLoad((float *)P0);
			VectorRegister4f R1 = VectorLoad((float *)P1);
			P0 += Inc0;
			P1 += Inc1;
			VectorRegister4f Res = fn(BatchState, R0, R1);
			VectorStore(Res, (float *)P2);
			VectorStore(Res, (float *)(P2 + P23d));
			P2 += sizeof(FVecReg);
		} while (P2 < End);
	}
	else
	{
		VectorRegister4f R0 = VectorLoad((float *)P0);
		VectorRegister4f R1 = VectorLoad((float *)P1);
		VectorRegister4f Res = fn(BatchState, R0, R1);
		VectorStore(Res, (float *)P2);
		VectorStore(Res, (float *)P3);
	}
	return InsPtr + 9;
}

VM_FORCEINLINE const uint8 *VVM_Dispatch_execFn1i_1i(const bool CT_MultipleLoops, const uint8 *InsPtr, FVectorVMBatchState *BatchState, FVectorVMExecContext *ExecCtx, VVMFn_1i fn, int NumLoops)
{
	uint16 *RegIndices = (uint16 *)InsPtr;
	uint8 *P0 = BatchState->RegPtrTable[RegIndices[0]];
	uint8 *P1 = BatchState->RegPtrTable[RegIndices[1]];
	if (CT_MultipleLoops)
	{
		uint32 Inc0 = (uint32)BatchState->RegIncTable[RegIndices[0]];
		uint8 *End  = P1 + sizeof(FVecReg) * NumLoops;
		do
		{
			VectorRegister4i R0 = VectorIntLoad(P0);
			P0 += Inc0;
			VectorRegister4i Res = fn(BatchState, R0);
			VectorIntStore(Res, P1);
			P1 += sizeof(FVecReg);
		} while (P1 < End);
	}
	else
	{
		VectorRegister4i R0 = VectorIntLoad(P0);
		VectorRegister4i Res = fn(BatchState, R0);
		VectorIntStore(Res, P1);
	}
	return InsPtr + 5;
}

VM_FORCEINLINE const uint8 *VVM_Dispatch_execFn2i_1i(const bool CT_MultipleLoops, const uint8 *InsPtr, FVectorVMBatchState *BatchState, FVectorVMExecContext *ExecCtx, VVMFn_2i fn, int NumLoops)
{
	uint16 *RegIndices = (uint16 *)InsPtr;
	uint8 *P0 = BatchState->RegPtrTable[RegIndices[0]];
	uint8 *P1 = BatchState->RegPtrTable[RegIndices[1]];
	uint8 *P2 = BatchState->RegPtrTable[RegIndices[2]];
	if (CT_MultipleLoops)
	{
		uint32 Inc0 = (uint32)BatchState->RegIncTable[RegIndices[0]];
		uint32 Inc1 = (uint32)BatchState->RegIncTable[RegIndices[1]];
		uint8 *End  = P2 + sizeof(FVecReg) * NumLoops;
		do
		{
			VectorRegister4i R0 = VectorIntLoad(P0);
			VectorRegister4i R1 = VectorIntLoad(P1);
			P0 += Inc0;
			P1 += Inc1;
			VectorRegister4i Res = fn(BatchState, R0, R1);
			VectorIntStore(Res, P2);
			P2 += sizeof(FVecReg);
		} while (P2 < End);
	}
	else
	{
		VectorRegister4i R0 = VectorIntLoad(P0);
		VectorRegister4i R1 = VectorIntLoad(P1);
		VectorRegister4i Res = fn(BatchState, R0, R1);
		VectorIntStore(Res, P2);
	}
	return InsPtr + 7;
}

VM_FORCEINLINE const uint8 *VVM_Dispatch_execFn3i_1i(const bool CT_MultipleLoops, const uint8 *InsPtr, FVectorVMBatchState *BatchState, FVectorVMExecContext *ExecCtx, VVMFn_3i fn, int NumLoops)
{
	uint16 *RegIndices = (uint16 *)InsPtr;
	uint8 *P0 = BatchState->RegPtrTable[RegIndices[0]];
	uint8 *P1 = BatchState->RegPtrTable[RegIndices[1]];
	uint8 *P2 = BatchState->RegPtrTable[RegIndices[2]];
	uint8 *P3 = BatchState->RegPtrTable[RegIndices[3]];
	if (CT_MultipleLoops)
	{
		uint32 Inc0 = (uint32)BatchState->RegIncTable[RegIndices[0]];
		uint32 Inc1 = (uint32)BatchState->RegIncTable[RegIndices[1]];
		uint32 Inc2 = (uint32)BatchState->RegIncTable[RegIndices[2]];
		uint8 *End  = P3 + sizeof(FVecReg) * NumLoops;
		do
		{
			VectorRegister4i R0 = VectorIntLoad(P0);
			VectorRegister4i R1 = VectorIntLoad(P1);
			VectorRegister4i R2 = VectorIntLoad(P2);
			P0 += Inc0;
			P1 += Inc1;
			P2 += Inc2;
			VectorRegister4i Res = fn(BatchState, R0, R1, R2);
			VectorIntStore(Res, P3);
			P3 += sizeof(FVecReg);
		} while (P3 < End);
	}
	else
	{
		VectorRegister4i R0 = VectorIntLoad(P0);
		VectorRegister4i R1 = VectorIntLoad(P1);
		VectorRegister4i R2 = VectorIntLoad(P2);
		VectorRegister4i Res = fn(BatchState, R0, R1, R2);
		VectorIntStore(Res, P3);
	}
	return InsPtr + 9;
}

VM_FORCEINLINE const uint8 *VVM_Dispatch_execFn1i_2i(const bool CT_MultipleLoops, const uint8 *InsPtr, FVectorVMBatchState *BatchState, FVectorVMExecContext *ExecCtx, VVMFn_1i fn, int NumLoops)
{
	uint16 *RegIndices = (uint16 *)InsPtr;
	uint8 *P0 = BatchState->RegPtrTable[RegIndices[0]];
	uint8 *P1 = BatchState->RegPtrTable[RegIndices[1]];
	uint8 *P2 = BatchState->RegPtrTable[RegIndices[2]];
	if (CT_MultipleLoops)
	{
		uint32 Inc0 = (uint32)BatchState->RegIncTable[RegIndices[0]];
		uint8 *End  = P1 + sizeof(FVecReg) * NumLoops;
		do
		{
			VectorRegister4i R0 = VectorIntLoad(P0);
			P0 += Inc0;
			VectorRegister4i Res = fn(BatchState, R0);
			VectorIntStore(Res, P1);
			VectorIntStore(Res, P2);
			P1 += sizeof(FVecReg);
			P2 += sizeof(FVecReg);
		} while (P2 < End);
	}
	else
	{
		VectorRegister4i R0 = VectorIntLoad(P0);
		VectorRegister4i Res = fn(BatchState, R0);
		VectorIntStore(Res, P1);
		VectorIntStore(Res, P2);
	}
	return InsPtr + 7;
}

VM_FORCEINLINE VectorRegister4f VVM_Exec2f_add                             (FVectorVMBatchState *BatchState, VectorRegister4f a, VectorRegister4f b)                                                            { return VectorAdd(a, b); }
VM_FORCEINLINE VectorRegister4f VVM_Exec2f_sub                             (FVectorVMBatchState *BatchState, VectorRegister4f a, VectorRegister4f b)                                                            { return VectorSubtract(a, b); }
VM_FORCEINLINE VectorRegister4f VVM_Exec2f_mul                             (FVectorVMBatchState *BatchState, VectorRegister4f a, VectorRegister4f b)                                                            { return VectorMultiply(a, b); }
VM_FORCEINLINE VectorRegister4f VVM_Exec2f_div                             (FVectorVMBatchState *BatchState, VectorRegister4f a, VectorRegister4f b)                                                            { return VectorSelect(VectorCompareGT(VectorAbs(b), VVM_m128Const(Epsilon)), VectorDivide(a, b), VectorZeroFloat()); }
VM_FORCEINLINE VectorRegister4f VVM_Exec3f_mad                             (FVectorVMBatchState *BatchState, VectorRegister4f a, VectorRegister4f b, VectorRegister4f c)                                        { return VectorMultiplyAdd(a, b, c); }
VM_FORCEINLINE VectorRegister4f VVM_Exec3f_lerp                            (FVectorVMBatchState *BatchState, VectorRegister4f a, VectorRegister4f b, VectorRegister4f c)                                        { return VectorLerp(a, b, c); }
VM_FORCEINLINE VectorRegister4f VVM_Exec1f_rcp                             (FVectorVMBatchState *BatchState, VectorRegister4f a)                                                                                { return VectorSelect(VectorCompareGT(VectorAbs(a) , VVM_m128Const(Epsilon)), VectorReciprocalEstimate(a), VectorZeroFloat()); }
VM_FORCEINLINE VectorRegister4f VVM_Exec1f_rsq                             (FVectorVMBatchState *BatchState, VectorRegister4f a)                                                                                { return VectorSelect(VectorCompareGT(a, VVM_m128Const(Epsilon)), VectorReciprocalSqrt(a), VectorZeroFloat()); }
VM_FORCEINLINE VectorRegister4f VVM_Exec1f_sqrt                            (FVectorVMBatchState *BatchState, VectorRegister4f a)                                                                                { return VectorSelect(VectorCompareGT(a, VVM_m128Const(Epsilon)), VectorSqrt(a), VectorZeroFloat()); }
VM_FORCEINLINE VectorRegister4f VVM_Exec1f_neg                             (FVectorVMBatchState *BatchState, VectorRegister4f a)                                                                                { return VectorNegate(a); }
VM_FORCEINLINE VectorRegister4f VVM_Exec1f_abs                             (FVectorVMBatchState *BatchState, VectorRegister4f a)                                                                                { return VectorAbs(a); }
VM_FORCEINLINE VectorRegister4f VVM_Exec1f_exp                             (FVectorVMBatchState *BatchState, VectorRegister4f a)                                                                                { return VectorExp(a); }
VM_FORCEINLINE VectorRegister4f VVM_Exec1f_exp2                            (FVectorVMBatchState *BatchState, VectorRegister4f a)                                                                                { return VectorExp2(a); }
VM_FORCEINLINE VectorRegister4f VVM_Exec1f_log                             (FVectorVMBatchState *BatchState, VectorRegister4f a)                                                                                { return VectorSelect(VectorCompareGT(a, VectorZeroFloat()), VectorLog(a), VectorZeroFloat()); }
VM_FORCEINLINE VectorRegister4f VVM_Exec1f_log2                            (FVectorVMBatchState *BatchState, VectorRegister4f a)                                                                                { return VectorLog2(a); }
VM_FORCEINLINE VectorRegister4f VVM_Exec1f_sin                             (FVectorVMBatchState *BatchState, VectorRegister4f a)                                                                                { return VectorSin(a); }
VM_FORCEINLINE VectorRegister4f VVM_Exec1f_cos                             (FVectorVMBatchState *BatchState, VectorRegister4f a)                                                                                { return VectorCos(a); }
VM_FORCEINLINE VectorRegister4f VVM_Exec1f_tan                             (FVectorVMBatchState *BatchState, VectorRegister4f a)                                                                                { return VectorTan(a); }
VM_FORCEINLINE VectorRegister4f VVM_Exec1f_acos                            (FVectorVMBatchState *BatchState, VectorRegister4f a)                                                                                { return VectorATan2(VVM_Exec1f_sqrt(BatchState, VectorMultiply(VectorSubtract(VVM_m128Const(One), a), VectorAdd(VVM_m128Const(One), a))), a); }
VM_FORCEINLINE VectorRegister4f VVM_Exec1f_asin                            (FVectorVMBatchState *BatchState, VectorRegister4f a)                                                                                { return VectorSubtract(VVM_m128Const(QuarterPi), VVM_Exec1f_acos(BatchState, a)); }
VM_FORCEINLINE VectorRegister4f VVM_Exec1f_atan                            (FVectorVMBatchState *BatchState, VectorRegister4f a)                                                                                { return VectorATan(a); }
VM_FORCEINLINE VectorRegister4f VVM_Exec2f_atan2                           (FVectorVMBatchState *BatchState, VectorRegister4f a, VectorRegister4f b)                                                            { return VectorATan2(a, b); }
VM_FORCEINLINE VectorRegister4f VVM_Exec1f_ceil                            (FVectorVMBatchState *BatchState, VectorRegister4f a)                                                                                { return VectorCeil(a); }
VM_FORCEINLINE VectorRegister4f VVM_Exec1f_floor                           (FVectorVMBatchState *BatchState, VectorRegister4f a)                                                                                { return VectorFloor(a); }
VM_FORCEINLINE VectorRegister4f VVM_Exec2f_fmod                            (FVectorVMBatchState *BatchState, VectorRegister4f a, VectorRegister4f b)                                                            { return VectorMod(a, b); }
VM_FORCEINLINE VectorRegister4f VVM_Exec1f_frac                            (FVectorVMBatchState *BatchState, VectorRegister4f a)                                                                                { return VectorFractional(a); }
VM_FORCEINLINE VectorRegister4f VVM_Exec1f_trunc                           (FVectorVMBatchState *BatchState, VectorRegister4f a)                                                                                { return VectorTruncate(a); }
VM_FORCEINLINE VectorRegister4f VVM_Exec3f_clamp                           (FVectorVMBatchState *BatchState, VectorRegister4f a, VectorRegister4f b, VectorRegister4f c)                                        { return VectorClamp(a, b, c); }
VM_FORCEINLINE VectorRegister4f VVM_Exec2f_min                             (FVectorVMBatchState *BatchState, VectorRegister4f a, VectorRegister4f b)                                                            { return VectorMin(a, b); }
VM_FORCEINLINE VectorRegister4f VVM_Exec2f_max                             (FVectorVMBatchState *BatchState, VectorRegister4f a, VectorRegister4f b)                                                            { return VectorMax(a, b); }
VM_FORCEINLINE VectorRegister4f VVM_Exec2f_pow                             (FVectorVMBatchState *BatchState, VectorRegister4f a, VectorRegister4f b)                                                            { return VectorSelect(VectorCompareGT(a, VVM_m128Const(Epsilon)), VectorPow(a, b), VectorZeroFloat()); }
VM_FORCEINLINE VectorRegister4f VVM_Exec1f_round                           (FVectorVMBatchState *BatchState, VectorRegister4f a)                                                                                { return VectorRound(a); }
VM_FORCEINLINE VectorRegister4f VVM_Exec1f_sign                            (FVectorVMBatchState *BatchState, VectorRegister4f a)                                                                                { return VectorSign(a); }
VM_FORCEINLINE VectorRegister4f VVM_Exec2f_step                            (FVectorVMBatchState *BatchState, VectorRegister4f a, VectorRegister4f b)                                                            { return VectorStep(VectorSubtract(b, a)); }
VM_FORCEINLINE const uint8 *    VVM_Exec1null_random                       (VVM_NULL_FN_ARGS)                                                                                                                   { return VVM_random(CT_MultipleLoops, InsPtr, BatchState, ExecCtx, NumLoops); }
VM_FORCEINLINE VectorRegister4f VVM_Exec1f_noise                           (FVectorVMBatchState* BatchState, VectorRegister4f a)																				{ return VVM_noise(a); }
VM_FORCEINLINE VectorRegister4f VVM_Exec2f_cmplt                           (FVectorVMBatchState *BatchState, VectorRegister4f a, VectorRegister4f b)                                                            { return VectorCompareLT(a, b); }
VM_FORCEINLINE VectorRegister4f VVM_Exec2f_cmple                           (FVectorVMBatchState *BatchState, VectorRegister4f a, VectorRegister4f b)                                                            { return VectorCompareLE(a, b); }
VM_FORCEINLINE VectorRegister4f VVM_Exec2f_cmpgt                           (FVectorVMBatchState *BatchState, VectorRegister4f a, VectorRegister4f b)                                                            { return VectorCompareGT(a, b); }
VM_FORCEINLINE VectorRegister4f VVM_Exec2f_cmpge                           (FVectorVMBatchState *BatchState, VectorRegister4f a, VectorRegister4f b)                                                            { return VectorCompareGE(a, b); }
VM_FORCEINLINE VectorRegister4f VVM_Exec2f_cmpeq                           (FVectorVMBatchState *BatchState, VectorRegister4f a, VectorRegister4f b)                                                            { return VectorCompareEQ(a, b); }
VM_FORCEINLINE VectorRegister4f VVM_Exec2f_cmpneq                          (FVectorVMBatchState *BatchState, VectorRegister4f a, VectorRegister4f b)                                                            { return VectorCompareNE(a, b); }
VM_FORCEINLINE VectorRegister4f VVM_Exec3f_select                          (FVectorVMBatchState *BatchState, VectorRegister4f a, VectorRegister4f b, VectorRegister4f c)                                        { return VectorSelect(a, b, c); }
VM_FORCEINLINE VectorRegister4i VVM_Exec2i_addi                            (FVectorVMBatchState *BatchState, VectorRegister4i a, VectorRegister4i b)                                                            { return VectorIntAdd(a, b); }
VM_FORCEINLINE VectorRegister4i VVM_Exec2i_subi                            (FVectorVMBatchState *BatchState, VectorRegister4i a, VectorRegister4i b)                                                            { return VectorIntSubtract(a, b); }
VM_FORCEINLINE VectorRegister4i VVM_Exec2i_muli                            (FVectorVMBatchState *BatchState, VectorRegister4i a, VectorRegister4i b)                                                            { return VectorIntMultiply(a, b); }
VM_FORCEINLINE VectorRegister4i VVM_Exec2i_divi                            (FVectorVMBatchState *BatchState, VectorRegister4i a, VectorRegister4i b)                                                            { return VVMIntDiv(a, b); }
VM_FORCEINLINE VectorRegister4i VVM_Exec3i_clampi                          (FVectorVMBatchState *BatchState, VectorRegister4i a, VectorRegister4i b, VectorRegister4i c)                                        { return VectorIntClamp(a, b, c); }
VM_FORCEINLINE VectorRegister4i VVM_Exec2i_mini                            (FVectorVMBatchState *BatchState, VectorRegister4i a, VectorRegister4i b)                                                            { return VectorIntMin(a, b); }
VM_FORCEINLINE VectorRegister4i VVM_Exec2i_maxi                            (FVectorVMBatchState *BatchState, VectorRegister4i a, VectorRegister4i b)                                                            { return VectorIntMax(a, b); }
VM_FORCEINLINE VectorRegister4i VVM_Exec1i_absi                            (FVectorVMBatchState *BatchState, VectorRegister4i a)                                                                                { return VectorIntAbs(a); }
VM_FORCEINLINE VectorRegister4i VVM_Exec1i_negi                            (FVectorVMBatchState *BatchState, VectorRegister4i a)                                                                                { return VectorIntNegate(a); }
VM_FORCEINLINE VectorRegister4i VVM_Exec1i_signi                           (FVectorVMBatchState *BatchState, VectorRegister4i a)                                                                                { return VectorIntSign(a); }
VM_FORCEINLINE const uint8 *    VVM_Exec1null_randomi                      (VVM_NULL_FN_ARGS)                                                                                                                   { return VVM_randomi(CT_MultipleLoops, InsPtr, BatchState, ExecCtx, NumLoops); }
VM_FORCEINLINE VectorRegister4i VVM_Exec2i_cmplti                          (FVectorVMBatchState *BatchState, VectorRegister4i a, VectorRegister4i b)                                                            { return VectorIntCompareLT(a, b); }
VM_FORCEINLINE VectorRegister4i VVM_Exec2i_cmplei                          (FVectorVMBatchState *BatchState, VectorRegister4i a, VectorRegister4i b)                                                            { return VectorIntCompareLE(a, b); }
VM_FORCEINLINE VectorRegister4i VVM_Exec2i_cmpgti                          (FVectorVMBatchState *BatchState, VectorRegister4i a, VectorRegister4i b)                                                            { return VectorIntCompareGT(a, b); }
VM_FORCEINLINE VectorRegister4i VVM_Exec2i_cmpgei                          (FVectorVMBatchState *BatchState, VectorRegister4i a, VectorRegister4i b)                                                            { return VectorIntCompareGE(a, b); }
VM_FORCEINLINE VectorRegister4i VVM_Exec2i_cmpeqi                          (FVectorVMBatchState *BatchState, VectorRegister4i a, VectorRegister4i b)                                                            { return VectorIntCompareEQ(a, b); }
VM_FORCEINLINE VectorRegister4i VVM_Exec2i_cmpneqi                         (FVectorVMBatchState *BatchState, VectorRegister4i a, VectorRegister4i b)                                                            { return VectorIntCompareNEQ(a, b); }
VM_FORCEINLINE VectorRegister4i VVM_Exec2i_bit_and                         (FVectorVMBatchState *BatchState, VectorRegister4i a, VectorRegister4i b)                                                            { return VectorIntAnd(a, b); }
VM_FORCEINLINE VectorRegister4i VVM_Exec2i_bit_or                          (FVectorVMBatchState *BatchState, VectorRegister4i a, VectorRegister4i b)                                                            { return VectorIntOr(a, b); }
VM_FORCEINLINE VectorRegister4i VVM_Exec2i_bit_xor                         (FVectorVMBatchState *BatchState, VectorRegister4i a, VectorRegister4i b)                                                            { return VectorIntXor(a, b); }
VM_FORCEINLINE VectorRegister4i VVM_Exec1i_bit_not                         (FVectorVMBatchState *BatchState, VectorRegister4i a)                                                                                { return VectorIntNot(a); }
VM_FORCEINLINE VectorRegister4i VVM_Exec2i_bit_lshift                      (FVectorVMBatchState *BatchState, VectorRegister4i a, VectorRegister4i b)                                                            { return VVMIntLShift(a, b); }
VM_FORCEINLINE VectorRegister4i VVM_Exec2i_bit_rshift                      (FVectorVMBatchState *BatchState, VectorRegister4i a, VectorRegister4i b)                                                            { return VVMIntRShift(a, b); }
VM_FORCEINLINE VectorRegister4i VVM_Exec2i_logic_and                       (FVectorVMBatchState *BatchState, VectorRegister4i a, VectorRegister4i b)                                                            { return VectorIntAnd(a, b); }
VM_FORCEINLINE VectorRegister4i VVM_Exec2i_logic_or                        (FVectorVMBatchState *BatchState, VectorRegister4i a, VectorRegister4i b)                                                            { return VectorIntOr(a, b); }
VM_FORCEINLINE VectorRegister4i VVM_Exec2i_logic_xor                       (FVectorVMBatchState *BatchState, VectorRegister4i a, VectorRegister4i b)                                                            { return VectorIntXor(a, b); }
VM_FORCEINLINE VectorRegister4i VVM_Exec1i_logic_not                       (FVectorVMBatchState *BatchState, VectorRegister4i a)                                                                                { return VectorIntNot(a); }
VM_FORCEINLINE VectorRegister4i VVM_Exec1i_f2i                             (FVectorVMBatchState *BatchState, VectorRegister4i a)                                                                                { return VVMf2i(a); }
VM_FORCEINLINE VectorRegister4f VVM_Exec1f_i2f                             (FVectorVMBatchState *BatchState, VectorRegister4f a)                                                                                { return VVMi2f(a); }
VM_FORCEINLINE VectorRegister4f VVM_Exec1f_f2b                             (FVectorVMBatchState *BatchState, VectorRegister4f a)                                                                                { return VectorCompareGT(a, VectorZeroFloat()); }
VM_FORCEINLINE VectorRegister4f VVM_Exec1f_b2f                             (FVectorVMBatchState *BatchState, VectorRegister4f a)                                                                                { return VectorSelect(a, VVM_m128Const(One), VectorZeroFloat()); }
VM_FORCEINLINE VectorRegister4i VVM_Exec1i_i2b                             (FVectorVMBatchState *BatchState, VectorRegister4i a)                                                                                { return VectorIntCompareGT(a, VectorSetZero()); }
VM_FORCEINLINE VectorRegister4i VVM_Exec1i_b2i                             (FVectorVMBatchState *BatchState, VectorRegister4i a)                                                                                { return VectorIntSelect(a, VectorIntSet1(1), VectorSetZero()); }
VM_FORCEINLINE const uint8 *    VVM_Exec0null_outputdata_float             (VVM_NULL_FN_ARGS)                                                                                                                   { return VVM_Output32(CT_MultipleLoops, InsPtr, BatchState, ExecCtx); }
VM_FORCEINLINE const uint8 *    VVM_Exec0null_outputdata_int32             (VVM_NULL_FN_ARGS)                                                                                                                   { return VVM_Output32(CT_MultipleLoops, InsPtr, BatchState, ExecCtx); }
VM_FORCEINLINE const uint8 *    VVM_Exec0null_outputdata_half              (VVM_NULL_FN_ARGS)                                                                                                                   { return VVM_Output16(CT_MultipleLoops, InsPtr, BatchState, ExecCtx); }
VM_FORCEINLINE const uint8 *    VVM_Exec1null_acquireindex                 (VVM_NULL_FN_ARGS)                                                                                                                   { return VVM_acquireindex(CT_MultipleLoops, InsPtr, BatchState, ExecCtx); }
VM_FORCEINLINE const uint8 *    VVM_Exec0null_external_func_call           (VVM_NULL_FN_ARGS)                                                                                                                   { return VVM_external_func_call(CT_MultipleLoops, InsPtr, BatchState, ExecCtx, NumLoops); }
VM_FORCEINLINE const uint8 *    VVM_Exec1null_exec_index                   (VVM_NULL_FN_ARGS)                                                                                                                   { return VVM_exec_index(CT_MultipleLoops, InsPtr, BatchState, NumLoops); }
VM_FORCEINLINE VectorRegister4f VVM_Exec2f_noise2D                         (FVectorVMBatchState* BatchState, VectorRegister4f x, VectorRegister4f y)															{ return VVM_noise(x, y); }
VM_FORCEINLINE VectorRegister4f VVM_Exec3f_noise3D                         (FVectorVMBatchState* BatchState, VectorRegister4f x, VectorRegister4f y, VectorRegister4f z)										{ return VVM_noise(x, y, z); }
VM_FORCEINLINE const uint8 *    VVM_Exec1null_enter_stat_scope             (VVM_NULL_FN_ARGS)                                                                                                                   { return InsPtr + 2; }
VM_FORCEINLINE const uint8 *    VVM_Exec0null_exit_stat_scope              (VVM_NULL_FN_ARGS)                                                                                                                   { return InsPtr; }
VM_FORCEINLINE const uint8 *    VVM_Exec0null_update_id                    (VVM_NULL_FN_ARGS)                                                                                                                   { return VVM_update_id(CT_MultipleLoops, InsPtr, BatchState, ExecCtx, NumLoops); }
VM_FORCEINLINE const uint8 *    VVM_Exec1null_acquire_id                   (VVM_NULL_FN_ARGS)                                                                                                                   { return VVM_acquire_id(CT_MultipleLoops, InsPtr, BatchState, ExecCtx, NumLoops); }
VM_FORCEINLINE const uint8 *    VVM_Exec1null_half_to_float                (VVM_NULL_FN_ARGS)                                                                                                                   { return VVM_half_to_float(CT_MultipleLoops, InsPtr, BatchState, NumLoops); }
VM_FORCEINLINE const uint8 *    VVM_Exec1null_fasi                         (VVM_NULL_FN_ARGS)                                                                                                                   { return InsPtr; }
VM_FORCEINLINE const uint8 *    VVM_Exec1null_iasf                         (VVM_NULL_FN_ARGS)                                                                                                                   { return InsPtr; }
VM_FORCEINLINE const uint8 *    VVM_Exec1null_exec_indexf                  (VVM_NULL_FN_ARGS)                                                                                                                   { return VVM_exec_indexf(CT_MultipleLoops, InsPtr, BatchState, NumLoops); }
VM_FORCEINLINE const uint8 *    VVM_Exec1null_exec_index_addi              (VVM_NULL_FN_ARGS)                                                                                                                   { return VVM_exec_index_addi(CT_MultipleLoops, InsPtr, BatchState, NumLoops); }
VM_FORCEINLINE VectorRegister4f VVM_Exec4f_cmplt_select                    (FVectorVMBatchState *BatchState, VectorRegister4f a, VectorRegister4f b, VectorRegister4f c, VectorRegister4f d)                    { return VectorSelect(VectorCompareLT(a, b), c, d); }
VM_FORCEINLINE VectorRegister4f VVM_Exec4f_cmple_select                    (FVectorVMBatchState *BatchState, VectorRegister4f a, VectorRegister4f b, VectorRegister4f c, VectorRegister4f d)                    { return VectorSelect(VectorCompareLE(a, b), c, d); }
VM_FORCEINLINE VectorRegister4f VVM_Exec4f_cmpeq_select                    (FVectorVMBatchState *BatchState, VectorRegister4f a, VectorRegister4f b, VectorRegister4f c, VectorRegister4f d)                    { return VectorSelect(VectorCompareEQ(a, b), c, d); }
VM_FORCEINLINE VectorRegister4f VVM_Exec4f_cmplti_select                   (FVectorVMBatchState *BatchState, VectorRegister4f a, VectorRegister4f b, VectorRegister4f c, VectorRegister4f d)                    { return VectorSelect(VectorCastIntToFloat(VectorIntCompareLT(*(VectorRegister4i *)&a, *(VectorRegister4i *)&b)), c, d); }
VM_FORCEINLINE VectorRegister4f VVM_Exec4f_cmplei_select                   (FVectorVMBatchState *BatchState, VectorRegister4f a, VectorRegister4f b, VectorRegister4f c, VectorRegister4f d)                    { return VectorSelect(VectorCastIntToFloat(VectorIntCompareLE(*(VectorRegister4i *)&a, *(VectorRegister4i *)&b)), c, d); }
VM_FORCEINLINE VectorRegister4f VVM_Exec4f_cmpeqi_select                   (FVectorVMBatchState *BatchState, VectorRegister4f a, VectorRegister4f b, VectorRegister4f c, VectorRegister4f d)                    { return VectorSelect(VectorCastIntToFloat(VectorIntCompareEQ(*(VectorRegister4i *)&a, *(VectorRegister4i *)&b)), c, d); }
VM_FORCEINLINE VectorRegister4f VVM_Exec3f_cmplt_logic_and                 (FVectorVMBatchState *BatchState, VectorRegister4f a, VectorRegister4f b, VectorRegister4f c)                                        { return VectorCastIntToFloat(VectorIntAnd(VectorCastFloatToInt(VectorCompareLT(a, b)), *(VectorRegister4i *)&c)); }
VM_FORCEINLINE VectorRegister4f VVM_Exec3f_cmple_logic_and                 (FVectorVMBatchState *BatchState, VectorRegister4f a, VectorRegister4f b, VectorRegister4f c)                                        { return VectorCastIntToFloat(VectorIntAnd(VectorCastFloatToInt(VectorCompareLE(a, b)), *(VectorRegister4i *)&c)); }
VM_FORCEINLINE VectorRegister4f VVM_Exec3f_cmpgt_logic_and                 (FVectorVMBatchState *BatchState, VectorRegister4f a, VectorRegister4f b, VectorRegister4f c)                                        { return VectorCastIntToFloat(VectorIntAnd(VectorCastFloatToInt(VectorCompareGT(a, b)), *(VectorRegister4i *)&c)); }
VM_FORCEINLINE VectorRegister4f VVM_Exec3f_cmpge_logic_and                 (FVectorVMBatchState *BatchState, VectorRegister4f a, VectorRegister4f b, VectorRegister4f c)                                        { return VectorCastIntToFloat(VectorIntAnd(VectorCastFloatToInt(VectorCompareGE(a, b)), *(VectorRegister4i *)&c)); }
VM_FORCEINLINE VectorRegister4f VVM_Exec3f_cmpeq_logic_and                 (FVectorVMBatchState *BatchState, VectorRegister4f a, VectorRegister4f b, VectorRegister4f c)                                        { return VectorCastIntToFloat(VectorIntAnd(VectorCastFloatToInt(VectorCompareEQ(a, b)), *(VectorRegister4i *)&c)); }
VM_FORCEINLINE VectorRegister4f VVM_Exec3f_cmpne_logic_and                 (FVectorVMBatchState *BatchState, VectorRegister4f a, VectorRegister4f b, VectorRegister4f c)                                        { return VectorCastIntToFloat(VectorIntAnd(VectorCastFloatToInt(VectorCompareNE(a, b)), *(VectorRegister4i *)&c)); }
VM_FORCEINLINE VectorRegister4i VVM_Exec3i_cmplti_logic_and                (FVectorVMBatchState *BatchState, VectorRegister4i a, VectorRegister4i b, VectorRegister4i c)                                        { return VectorIntAnd(VectorIntCompareLT(a, b), c); }
VM_FORCEINLINE VectorRegister4i VVM_Exec3i_cmplei_logic_and                (FVectorVMBatchState *BatchState, VectorRegister4i a, VectorRegister4i b, VectorRegister4i c)                                        { return VectorIntAnd(VectorIntCompareLE(a, b), c); }
VM_FORCEINLINE VectorRegister4i VVM_Exec3i_cmpgti_logic_and                (FVectorVMBatchState *BatchState, VectorRegister4i a, VectorRegister4i b, VectorRegister4i c)                                        { return VectorIntAnd(VectorIntCompareGT(a, b), c); }
VM_FORCEINLINE VectorRegister4i VVM_Exec3i_cmpgei_logic_and                (FVectorVMBatchState *BatchState, VectorRegister4i a, VectorRegister4i b, VectorRegister4i c)                                        { return VectorIntAnd(VectorIntCompareGE(a, b), c); }
VM_FORCEINLINE VectorRegister4i VVM_Exec3i_cmpeqi_logic_and                (FVectorVMBatchState *BatchState, VectorRegister4i a, VectorRegister4i b, VectorRegister4i c)                                        { return VectorIntAnd(VectorIntCompareEQ(a, b), c); }
VM_FORCEINLINE VectorRegister4i VVM_Exec3i_cmpnei_logic_and                (FVectorVMBatchState *BatchState, VectorRegister4i a, VectorRegister4i b, VectorRegister4i c)                                        { return VectorIntAnd(VectorIntCompareNEQ(a, b), c); }
VM_FORCEINLINE VectorRegister4f VVM_Exec3f_cmplt_logic_or                  (FVectorVMBatchState *BatchState, VectorRegister4f a, VectorRegister4f b, VectorRegister4f c)                                        { return VectorCastIntToFloat(VectorIntOr(VectorCastFloatToInt(VectorCompareLT(a, b)), *(VectorRegister4i *)&c)); }
VM_FORCEINLINE VectorRegister4f VVM_Exec3f_cmple_logic_or                  (FVectorVMBatchState *BatchState, VectorRegister4f a, VectorRegister4f b, VectorRegister4f c)                                        { return VectorCastIntToFloat(VectorIntOr(VectorCastFloatToInt(VectorCompareLE(a, b)), *(VectorRegister4i *)&c)); }
VM_FORCEINLINE VectorRegister4f VVM_Exec3f_cmpgt_logic_or                  (FVectorVMBatchState *BatchState, VectorRegister4f a, VectorRegister4f b, VectorRegister4f c)                                        { return VectorCastIntToFloat(VectorIntOr(VectorCastFloatToInt(VectorCompareGT(a, b)), *(VectorRegister4i *)&c)); }
VM_FORCEINLINE VectorRegister4f VVM_Exec3f_cmpge_logic_or                  (FVectorVMBatchState *BatchState, VectorRegister4f a, VectorRegister4f b, VectorRegister4f c)                                        { return VectorCastIntToFloat(VectorIntOr(VectorCastFloatToInt(VectorCompareGE(a, b)), *(VectorRegister4i *)&c)); }
VM_FORCEINLINE VectorRegister4f VVM_Exec3f_cmpeq_logic_or                  (FVectorVMBatchState *BatchState, VectorRegister4f a, VectorRegister4f b, VectorRegister4f c)                                        { return VectorCastIntToFloat(VectorIntOr(VectorCastFloatToInt(VectorCompareEQ(a, b)), *(VectorRegister4i *)&c)); }
VM_FORCEINLINE VectorRegister4f VVM_Exec3f_cmpne_logic_or                  (FVectorVMBatchState *BatchState, VectorRegister4f a, VectorRegister4f b, VectorRegister4f c)                                        { return VectorCastIntToFloat(VectorIntOr(VectorCastFloatToInt(VectorCompareNE(a, b)), *(VectorRegister4i *)&c)); }
VM_FORCEINLINE VectorRegister4i VVM_Exec3i_cmplti_logic_or                 (FVectorVMBatchState *BatchState, VectorRegister4i a, VectorRegister4i b, VectorRegister4i c)                                        { return VectorIntOr(VectorIntCompareLT(a, b), c); }
VM_FORCEINLINE VectorRegister4i VVM_Exec3i_cmplei_logic_or                 (FVectorVMBatchState *BatchState, VectorRegister4i a, VectorRegister4i b, VectorRegister4i c)                                        { return VectorIntOr(VectorIntCompareLE(a, b), c); }
VM_FORCEINLINE VectorRegister4i VVM_Exec3i_cmpgti_logic_or                 (FVectorVMBatchState *BatchState, VectorRegister4i a, VectorRegister4i b, VectorRegister4i c)                                        { return VectorIntOr(VectorIntCompareGT(a, b), c); }
VM_FORCEINLINE VectorRegister4i VVM_Exec3i_cmpgei_logic_or                 (FVectorVMBatchState *BatchState, VectorRegister4i a, VectorRegister4i b, VectorRegister4i c)                                        { return VectorIntOr(VectorIntCompareGE(a, b), c); }
VM_FORCEINLINE VectorRegister4i VVM_Exec3i_cmpeqi_logic_or                 (FVectorVMBatchState *BatchState, VectorRegister4i a, VectorRegister4i b, VectorRegister4i c)                                        { return VectorIntOr(VectorIntCompareEQ(a, b), c); }
VM_FORCEINLINE VectorRegister4i VVM_Exec3i_cmpnei_logic_or                 (FVectorVMBatchState *BatchState, VectorRegister4i a, VectorRegister4i b, VectorRegister4i c)                                        { return VectorIntOr(VectorIntCompareNEQ(a, b), c); }
VM_FORCEINLINE VectorRegister4f VVM_Exec4f_mad_add                         (FVectorVMBatchState *BatchState, VectorRegister4f a, VectorRegister4f b, VectorRegister4f c, VectorRegister4f d)                    { return VectorAdd(VectorMultiplyAdd(a, b, c), d); }
VM_FORCEINLINE VectorRegister4f VVM_Exec4f_mad_sub0                        (FVectorVMBatchState *BatchState, VectorRegister4f a, VectorRegister4f b, VectorRegister4f c, VectorRegister4f d)                    { return VectorSubtract(VectorMultiplyAdd(a, b, c), d); }
VM_FORCEINLINE VectorRegister4f VVM_Exec4f_mad_sub1                        (FVectorVMBatchState *BatchState, VectorRegister4f a, VectorRegister4f b, VectorRegister4f c, VectorRegister4f d)                    { return VectorSubtract(d, VectorMultiplyAdd(a, b, c)); }
VM_FORCEINLINE VectorRegister4f VVM_Exec4f_mad_mul                         (FVectorVMBatchState *BatchState, VectorRegister4f a, VectorRegister4f b, VectorRegister4f c, VectorRegister4f d)                    { return VectorMultiply(VectorMultiplyAdd(a, b, c), d); }
VM_FORCEINLINE VectorRegister4f VVM_Exec3f_mad_sqrt                        (FVectorVMBatchState *BatchState, VectorRegister4f a, VectorRegister4f b, VectorRegister4f c)                                        { return VectorSqrt(VectorMultiplyAdd(a, b, c)); }
VM_FORCEINLINE VectorRegister4f VVM_Exec5f_mad_mad0                        (FVectorVMBatchState *BatchState, VectorRegister4f a, VectorRegister4f b, VectorRegister4f c, VectorRegister4f d, VectorRegister4f e){ return VectorMultiplyAdd(d, e, VectorMultiplyAdd(a, b, c)); }
VM_FORCEINLINE VectorRegister4f VVM_Exec5f_mad_mad1                        (FVectorVMBatchState *BatchState, VectorRegister4f a, VectorRegister4f b, VectorRegister4f c, VectorRegister4f d, VectorRegister4f e){ return VectorMultiplyAdd(VectorMultiplyAdd(a, b, c), d, e); }
VM_FORCEINLINE VectorRegister4f VVM_Exec4f_mul_mad0                        (FVectorVMBatchState *BatchState, VectorRegister4f a, VectorRegister4f b, VectorRegister4f c, VectorRegister4f d)                    { return VectorMultiplyAdd(VectorMultiply(a, b), c, d); }
VM_FORCEINLINE VectorRegister4f VVM_Exec4f_mul_mad1                        (FVectorVMBatchState *BatchState, VectorRegister4f a, VectorRegister4f b, VectorRegister4f c, VectorRegister4f d)                    { return VectorMultiplyAdd(c, d, VectorMultiply(a, b)); }
VM_FORCEINLINE VectorRegister4f VVM_Exec3f_mul_add                         (FVectorVMBatchState *BatchState, VectorRegister4f a, VectorRegister4f b, VectorRegister4f c)                                        { return VectorAdd(VectorMultiply(a, b), c); }
VM_FORCEINLINE VectorRegister4f VVM_Exec3f_mul_sub0                        (FVectorVMBatchState *BatchState, VectorRegister4f a, VectorRegister4f b, VectorRegister4f c)                                        { return VectorSubtract(VectorMultiply(a, b), c); }
VM_FORCEINLINE VectorRegister4f VVM_Exec3f_mul_sub1                        (FVectorVMBatchState *BatchState, VectorRegister4f a, VectorRegister4f b, VectorRegister4f c)                                        { return VectorSubtract(c, VectorMultiply(a, b)); }
VM_FORCEINLINE VectorRegister4f VVM_Exec3f_mul_mul                         (FVectorVMBatchState *BatchState, VectorRegister4f a, VectorRegister4f b, VectorRegister4f c)                                        { return VectorMultiply(VectorMultiply(a, b), c); }
VM_FORCEINLINE VectorRegister4f VVM_Exec3f_mul_max                         (FVectorVMBatchState *BatchState, VectorRegister4f a, VectorRegister4f b, VectorRegister4f c)                                        { return VectorMax(VectorMultiply(a, b), c); }
VM_FORCEINLINE VectorRegister4f VVM_Exec2f_mul_2x                          (FVectorVMBatchState *BatchState, VectorRegister4f a, VectorRegister4f b)                                                            { return VectorMultiply(a, b); }
VM_FORCEINLINE VectorRegister4f VVM_Exec4f_add_mad1                        (FVectorVMBatchState *BatchState, VectorRegister4f a, VectorRegister4f b, VectorRegister4f c, VectorRegister4f d)                    { return VectorMultiplyAdd(c, d, VectorAdd(a, b)); }
VM_FORCEINLINE VectorRegister4f VVM_Exec3f_add_add                         (FVectorVMBatchState *BatchState, VectorRegister4f a, VectorRegister4f b, VectorRegister4f c)                                        { return VectorAdd(VectorAdd(a, b), c); }
VM_FORCEINLINE VectorRegister4f VVM_Exec3f_sub_cmplt1                      (FVectorVMBatchState *BatchState, VectorRegister4f a, VectorRegister4f b, VectorRegister4f c)                                        { return VectorCompareLT(c, VectorSubtract(a, b)); }
VM_FORCEINLINE VectorRegister4f VVM_Exec2f_sub_neg                         (FVectorVMBatchState *BatchState, VectorRegister4f a, VectorRegister4f b)                                                            { return VectorNegate(VectorSubtract(a, b)); }
VM_FORCEINLINE VectorRegister4f VVM_Exec3f_sub_mul                         (FVectorVMBatchState *BatchState, VectorRegister4f a, VectorRegister4f b, VectorRegister4f c)                                        { return VectorMultiply(VectorSubtract(a, b), c); }
VM_FORCEINLINE VectorRegister4f VVM_Exec4f_div_mad0                        (FVectorVMBatchState *BatchState, VectorRegister4f a, VectorRegister4f b, VectorRegister4f c, VectorRegister4f d)                    { return VectorMultiplyAdd(VVM_Exec2f_div(BatchState, a, b), c, d); }
VM_FORCEINLINE VectorRegister4i VVM_Exec2i_div_f2i                         (FVectorVMBatchState *BatchState, VectorRegister4i a, VectorRegister4i b)                                                            { return VectorFloatToInt(VVM_Exec2f_div(BatchState, VectorCastIntToFloat(a), VectorCastIntToFloat(b))); }
VM_FORCEINLINE VectorRegister4f VVM_Exec3f_div_mul                         (FVectorVMBatchState *BatchState, VectorRegister4f a, VectorRegister4f b, VectorRegister4f c)                                        { return VectorMultiply(VVM_Exec2f_div(BatchState, a, b), c); }
VM_FORCEINLINE VectorRegister4i VVM_Exec3i_muli_addi                       (FVectorVMBatchState *BatchState, VectorRegister4i a, VectorRegister4i b, VectorRegister4i c)                                        { return VectorIntAdd(VectorIntMultiply(a, b), c); }
VM_FORCEINLINE VectorRegister4i VVM_Exec3i_addi_bit_rshift                 (FVectorVMBatchState *BatchState, VectorRegister4i a, VectorRegister4i b, VectorRegister4i c)                                        { return VVMIntRShift(VectorIntAdd(a, b), c); }
VM_FORCEINLINE VectorRegister4i VVM_Exec3i_addi_muli                       (FVectorVMBatchState *BatchState, VectorRegister4i a, VectorRegister4i b, VectorRegister4i c)                                        { return VectorIntMultiply(VectorIntAdd(a, b), c); }
VM_FORCEINLINE VectorRegister4i VVM_Exec1i_b2i_2x                          (FVectorVMBatchState *BatchState, VectorRegister4i a)                                                                                { return VectorIntSelect(a, VectorIntSet1(1), VectorSetZero()); }
VM_FORCEINLINE VectorRegister4f VVM_Exec2f_i2f_div0                        (FVectorVMBatchState *BatchState, VectorRegister4f a, VectorRegister4f b)                                                            { return VVM_Exec2f_div(BatchState, VectorIntToFloat(VectorCastFloatToInt(a)), b); }
VM_FORCEINLINE VectorRegister4f VVM_Exec2f_i2f_div1                        (FVectorVMBatchState *BatchState, VectorRegister4f a, VectorRegister4f b)                                                            { return VVM_Exec2f_div(BatchState, b, VectorIntToFloat(VectorCastFloatToInt(a))); }
VM_FORCEINLINE VectorRegister4f VVM_Exec2f_i2f_mul                         (FVectorVMBatchState *BatchState, VectorRegister4f a, VectorRegister4f b)                                                            { return VectorMultiply(VectorIntToFloat(VectorCastFloatToInt(a)), b); }
VM_FORCEINLINE VectorRegister4f VVM_Exec3f_i2f_mad0                        (FVectorVMBatchState *BatchState, VectorRegister4f a, VectorRegister4f b, VectorRegister4f c)                                        { return VectorMultiplyAdd(VectorIntToFloat(VectorCastFloatToInt(a)), b, c); }
VM_FORCEINLINE VectorRegister4f VVM_Exec3f_i2f_mad1                        (FVectorVMBatchState *BatchState, VectorRegister4f a, VectorRegister4f b, VectorRegister4f c)                                        { return VectorMultiplyAdd(a, b, VectorIntToFloat(VectorCastFloatToInt(c))); }
VM_FORCEINLINE VectorRegister4i VVM_Exec3i_f2i_select1                     (FVectorVMBatchState *BatchState, VectorRegister4i a, VectorRegister4i b, VectorRegister4i c)                                        { return VectorIntSelect(a, VectorFloatToInt(VectorCastIntToFloat(b)), c); }
VM_FORCEINLINE VectorRegister4i VVM_Exec2i_f2i_maxi                        (FVectorVMBatchState *BatchState, VectorRegister4i a, VectorRegister4i b)                                                            { return VectorIntMax(VectorFloatToInt(VectorCastIntToFloat(a)), b); }
VM_FORCEINLINE VectorRegister4i VVM_Exec2i_f2i_addi                        (FVectorVMBatchState *BatchState, VectorRegister4i a, VectorRegister4i b)                                                            { return VectorIntAdd(VectorFloatToInt(VectorCastIntToFloat(a)), b); }
VM_FORCEINLINE VectorRegister4f VVM_Exec3f_fmod_add                        (FVectorVMBatchState *BatchState, VectorRegister4f a, VectorRegister4f b, VectorRegister4f c)                                        { return VectorAdd(VectorMod(a, b), c); }
VM_FORCEINLINE VectorRegister4f VVM_Exec2f_bit_and_i2f                     (FVectorVMBatchState *BatchState, VectorRegister4f a, VectorRegister4f b)                                                            { return VectorIntToFloat(VectorIntAnd(VectorCastFloatToInt(a), VectorCastFloatToInt(b))); }
VM_FORCEINLINE VectorRegister4i VVM_Exec3i_bit_rshift_bit_and              (FVectorVMBatchState *BatchState, VectorRegister4i a, VectorRegister4i b, VectorRegister4i c)                                        { return VectorIntAnd(VVMIntRShift(a, b), c); }
VM_FORCEINLINE VectorRegister4f VVM_Exec2f_neg_cmplt                       (FVectorVMBatchState *BatchState, VectorRegister4f a, VectorRegister4f b)                                                            { return VectorCompareLT(VectorNegate(a), b); }
VM_FORCEINLINE VectorRegister4i VVM_Exec3i_bit_or_muli                     (FVectorVMBatchState *BatchState, VectorRegister4i a, VectorRegister4i b, VectorRegister4i c)                                        { return VectorIntMultiply(VectorIntOr(a, b), c); }
VM_FORCEINLINE VectorRegister4i VVM_Exec3i_bit_lshift_bit_or               (FVectorVMBatchState *BatchState, VectorRegister4i a, VectorRegister4i b, VectorRegister4i c)                                        { return VectorIntOr(VVMIntLShift(a, b), c); }
VM_FORCEINLINE const uint8 *    VVM_Exec2null_random_add                   (VVM_NULL_FN_ARGS)                                                                                                                   { return VVM_random_add(CT_MultipleLoops, InsPtr, BatchState, ExecCtx, NumLoops); }
VM_FORCEINLINE const uint8 *    VVM_Exec1null_random_2x                    (VVM_NULL_FN_ARGS)                                                                                                                   { return VVM_random_2x(CT_MultipleLoops, InsPtr, BatchState, ExecCtx, NumLoops); }
VM_FORCEINLINE VectorRegister4i VVM_Exec2i_max_f2i                         (FVectorVMBatchState *BatchState, VectorRegister4i a, VectorRegister4i b)                                                            { return VectorFloatToInt(VectorMax(VectorCastIntToFloat(a), VectorCastIntToFloat(b))); }
VM_FORCEINLINE VectorRegister4f VVM_Exec4f_select_mul                      (FVectorVMBatchState *BatchState, VectorRegister4f a, VectorRegister4f b, VectorRegister4f c, VectorRegister4f d)                    { return VectorMultiply(VectorSelect(a, b, c), d); }
VM_FORCEINLINE VectorRegister4f VVM_Exec4f_select_add                      (FVectorVMBatchState *BatchState, VectorRegister4f a, VectorRegister4f b, VectorRegister4f c, VectorRegister4f d)                    { return VectorAdd(VectorSelect(a, b, c), d); }
VM_FORCEINLINE const uint8 *    VVM_Exec1null_sin_cos                      (VVM_NULL_FN_ARGS)                                                                                                                   { return VVM_sincos(CT_MultipleLoops, InsPtr, BatchState, NumLoops); }
VM_FORCEINLINE const uint8 *    VVM_Exec0null_inputdata_float              (VVM_NULL_FN_ARGS)                                                                                                                   { return InsPtr; }
VM_FORCEINLINE const uint8 *    VVM_Exec0null_inputdata_int32              (VVM_NULL_FN_ARGS)                                                                                                                   { return InsPtr; }
VM_FORCEINLINE const uint8 *    VVM_Exec0null_inputdata_half               (VVM_NULL_FN_ARGS)                                                                                                                   { return InsPtr; }
VM_FORCEINLINE const uint8 *    VVM_Exec0null_inputdata_noadvance_float    (VVM_NULL_FN_ARGS)                                                                                                                   { return InsPtr; }
VM_FORCEINLINE const uint8 *    VVM_Exec0null_inputdata_noadvance_int32    (VVM_NULL_FN_ARGS)                                                                                                                   { return InsPtr; }
VM_FORCEINLINE const uint8 *    VVM_Exec0null_inputdata_noadvance_half     (VVM_NULL_FN_ARGS)                                                                                                                   { return InsPtr; }
VM_FORCEINLINE const uint8 *    VVM_Exec0null_outputdata_float_from_half   (VVM_NULL_FN_ARGS)                                                                                                                   { return VVM_Output32_from_16(CT_MultipleLoops, InsPtr, BatchState, ExecCtx); }
VM_FORCEINLINE const uint8 *    VVM_Exec0null_outputdata_half_from_half    (VVM_NULL_FN_ARGS)                                                                                                                   { return VVM_Output16_from_16(CT_MultipleLoops, InsPtr, BatchState, ExecCtx); }
//this is the macro required to exit the infinite loop running over the bytecode most efficiently
#define VVM_Dispatch_execFn0done_0done(...) NULL; goto done_loop;

#if defined(VECTORVM_SUPPORTS_COMPUTED_GOTO) && VECTORVM_SUPPORTS_COMPUTED_GOTO
#define VVM_OP_CASE(op)	jmp_lbl_##op:
#define VVM_OP_NEXT     goto *jmp_tbl[InsPtr[-1]]
#define VVM_OP_START	VVM_OP_NEXT;

static const void* jmp_tbl[] = {
#	define VVM_OP_XM(op, ...) &&jmp_lbl_##op,
	VVM_OP_XM_LIST
#	undef VVM_OP_XM
};
#else
#define VVM_OP_START    switch ((EVectorVMOp)InsPtr[-1])
#define VVM_OP_CASE(op)	case EVectorVMOp::op:
#define VVM_OP_NEXT     break
#endif

static void SetBatchPointersForCorrectChunkOffsets(FVectorVMExecContext *ExecCtx, FVectorVMBatchState *BatchState)
{
	{ //input buffers
		int DstOffset = ExecCtx->VVMState->NumTempRegisters + ExecCtx->VVMState->NumConstBuffers;
		int SrcOffset = DstOffset + ExecCtx->VVMState->NumInputBuffers;
		for (int i = 0; i < (int)ExecCtx->VVMState->NumInputBuffers; ++i)
		{
			if (BatchState->RegIncTable[DstOffset + i] != 0) //don't offset the no-advance inputs
			{
				const uint32 DataTypeStride = ExecCtx->VVMState->InputMapCacheSrc[i] & 0x4000 ? 2 : 4;
				const uint32 OffsetBytes = BatchState->ChunkLocalData.StartInstanceThisChunk * DataTypeStride;
				BatchState->RegPtrTable[DstOffset + i] = BatchState->RegPtrTable[SrcOffset + i] + OffsetBytes;
			}
		}
	}
	if (BatchState->ChunkLocalData.StartInstanceThisChunk != 0)
	{ //output buffers
		int Offset = ExecCtx->VVMState->NumTempRegisters + ExecCtx->VVMState->NumConstBuffers + ExecCtx->VVMState->NumInputBuffers * 2;
		for (int i = 0; i < (int)ExecCtx->VVMState->NumOutputBuffers; ++i)
		{
			const uint32 DataTypeStride = ExecCtx->VVMState->OutputRemapDataType[i] == 2 ? 2 : 4;
			const uint32 OffsetBytes = BatchState->ChunkLocalData.NumOutputPerDataSet[0] * DataTypeStride;
			BatchState->RegPtrTable[Offset + i] = BatchState->RegPtrTable[Offset + i] + OffsetBytes;
		}
	}
	for (uint32 i = 0; i < ExecCtx->VVMState->MaxOutputDataSet; ++i)
	{
		BatchState->ChunkLocalData.StartingOutputIdxPerDataSet[i] = 0;
		BatchState->ChunkLocalData.NumOutputPerDataSet[i] = 0;
	}
}

static void ExecChunkSingleLoop(FVectorVMExecContext *ExecCtx, FVectorVMBatchState *BatchState)
{
	static const int      NumLoops       = 1;
	static constexpr bool CT_MultipleLoops = false;

	SetBatchPointersForCorrectChunkOffsets(ExecCtx, BatchState);

	const uint8 *InsPtr        = ExecCtx->VVMState->Bytecode + 1;
	const uint8 *InsPtrEnd     = InsPtr + ExecCtx->VVMState->NumBytecodeBytes;

	for (;;)
	{
		VVM_OP_START
		{
#			define VVM_OP_XM(OpCode, Cat, NumInputs, NumOutputs, Type, ...) VVM_OP_CASE(OpCode) InsPtr = VVM_Dispatch_execFn##NumInputs##Type##_##NumOutputs##Type(CT_MultipleLoops, InsPtr, BatchState, ExecCtx, VVM_Exec##NumInputs##Type##_##OpCode, NumLoops); VVM_OP_NEXT;
			VVM_OP_XM_LIST
#			undef VVM_OP_XM
		}
	}
	done_loop: ; // breaking out of the above loop
}

static void ExecChunkMultipleLoops(FVectorVMExecContext *ExecCtx, FVectorVMBatchState *BatchState, int NumLoops)
{
	static constexpr bool CT_MultipleLoops = true;
	SetBatchPointersForCorrectChunkOffsets(ExecCtx, BatchState);

	const uint8 *InsPtr        = ExecCtx->VVMState->Bytecode + 1;
	const uint8 *InsPtrEnd     = InsPtr + ExecCtx->VVMState->NumBytecodeBytes;

	for (;;)
	{
		VVM_OP_START
		{
#			define VVM_OP_XM(OpCode, Cat, NumInputs, NumOutputs, Type, ...) VVM_OP_CASE(OpCode) InsPtr = VVM_Dispatch_execFn##NumInputs##Type##_##NumOutputs##Type(CT_MultipleLoops, InsPtr, BatchState, ExecCtx, VVM_Exec##NumInputs##Type##_##OpCode, NumLoops); VVM_OP_NEXT;
			VVM_OP_XM_LIST
#			undef VVM_OP_XM
		}
	}
	done_loop: ; // breaking out of the above loop
}

#undef VVM_Dispatch_execFn0done_0done

#undef VVM_OP_CASE
#undef VVM_OP_NEXT

void ExecVectorVMState(FVectorVMExecContext *ExecCtx)
{
	if (ExecCtx->ExtFunctionTable.Num() != ExecCtx->VVMState->NumExtFunctions)
	{
		return;
	}
	for (uint32 i = 0; i < ExecCtx->VVMState->NumExtFunctions; ++i)
	{
		ExecCtx->VVMState->ExtFunctionTable[i].Function = ExecCtx->ExtFunctionTable[i];
	}
	for (uint32 i = 0; i < ExecCtx->VVMState->MaxOutputDataSet; ++i)
	{
		ExecCtx->VVMState->NumOutputPerDataSet[i] = 0;
	}

	//cache the mappings from niagara data buffers to internal minimized set
	if (!(ExecCtx->VVMState->Flags & VVMFlag_DataMapCacheSetup))
	{
		VVMBuildMapTableCaches(ExecCtx);
		ExecCtx->VVMState->Flags |= VVMFlag_DataMapCacheSetup;
	}

	for (uint32 i = 0; i < ExecCtx->VVMState->NumConstBuffers; ++i)
	{
		ExecCtx->VVMState->ConstantBuffers[i].i = VectorIntSet1(((uint32 *)ExecCtx->ConstantTableData[ExecCtx->VVMState->ConstMapCacheIdx[i]])[ExecCtx->VVMState->ConstMapCacheSrc[i]]);
	}

	//if the number of instances hasn't changed since the last exec, we don't have to re-compute the internal state
	if (ExecCtx->NumInstances == ExecCtx->VVMState->NumInstancesExecCached) 
	{
		ExecCtx->Internal.NumBytesRequiredPerBatch          = ExecCtx->VVMState->ExecCtxCache.NumBytesRequiredPerBatch;
		ExecCtx->Internal.PerBatchRegisterDataBytesRequired = ExecCtx->VVMState->ExecCtxCache.PerBatchRegisterDataBytesRequired;
		ExecCtx->Internal.MaxChunksPerBatch                 = ExecCtx->VVMState->ExecCtxCache.MaxChunksPerBatch;
		ExecCtx->Internal.MaxInstancesPerChunk              = ExecCtx->VVMState->ExecCtxCache.MaxInstancesPerChunk;
	}
	else
	{ //calculate Batch & Chunk division and all internal execution state required before executing
		static const uint32 MaxChunksPerBatch                              = 4; //*MUST BE POW 2* arbitrary 4 chunks per batch... this is harder to load balance because it depends on CPU cores available during execution
		static_assert(MaxChunksPerBatch > 0 && (MaxChunksPerBatch & (MaxChunksPerBatch - 1)) == 0);

		size_t PageSizeInBytes = (uint64_t)GVVMPageSizeInKB << 10;

		size_t PerBatchRegisterDataBytesRequired                           = 0;
		int NumBatches                                                     = 1;
		int NumChunksPerBatch                                              = (int)MaxChunksPerBatch;
		uint32 MaxLoopsPerChunk                                            = 0;
		{ //compute the number of bytes required per batch
			const uint32 TotalNumLoopsRequired              = VVM_MAX(((uint32)ExecCtx->NumInstances + 3) >> 2, 1);
			const size_t NumBytesRequiredPerLoop            = VVM_REG_SIZE * ExecCtx->VVMState->NumTempRegisters + ExecCtx->VVMState->MaxOutputDataSet * sizeof(uint8);
			size_t NumBytesPerBatchAvailableForTempRegs = PageSizeInBytes - ExecCtx->VVMState->BatchOverheadSize;
			size_t TotalNumLoopBytesRequired            = VVM_ALIGN(TotalNumLoopsRequired, MaxChunksPerBatch) * NumBytesRequiredPerLoop;
			if (NumBytesPerBatchAvailableForTempRegs < TotalNumLoopBytesRequired)
			{
				//Not everything fits into a single chunk, so we have to compute everything here
				int NumChunksRequired                  = (int)(TotalNumLoopBytesRequired + NumBytesPerBatchAvailableForTempRegs - 1) / (int)NumBytesPerBatchAvailableForTempRegs;
				MaxLoopsPerChunk					   = (TotalNumLoopsRequired + NumChunksRequired - 1) / NumChunksRequired;
				NumChunksPerBatch = NumChunksRequired;
			}
			else
			{
				//everything fits into a single chunk
				NumChunksPerBatch = 1;
				MaxLoopsPerChunk = TotalNumLoopsRequired;
			}
			PerBatchRegisterDataBytesRequired = MaxLoopsPerChunk * NumBytesRequiredPerLoop;
		}

		size_t NumBytesRequiredPerBatch = ExecCtx->VVMState->BatchOverheadSize + PerBatchRegisterDataBytesRequired;
		ExecCtx->Internal.NumBytesRequiredPerBatch          = (uint32)NumBytesRequiredPerBatch;
		ExecCtx->Internal.PerBatchRegisterDataBytesRequired = (uint32)PerBatchRegisterDataBytesRequired;
		ExecCtx->Internal.MaxChunksPerBatch                 = NumChunksPerBatch;
		ExecCtx->Internal.MaxInstancesPerChunk              = MaxLoopsPerChunk << 2;

		ExecCtx->VVMState->ExecCtxCache.NumBytesRequiredPerBatch          = ExecCtx->Internal.NumBytesRequiredPerBatch;
		ExecCtx->VVMState->ExecCtxCache.PerBatchRegisterDataBytesRequired = ExecCtx->Internal.PerBatchRegisterDataBytesRequired;
		ExecCtx->VVMState->ExecCtxCache.MaxChunksPerBatch                 = ExecCtx->Internal.MaxChunksPerBatch;
		ExecCtx->VVMState->ExecCtxCache.MaxInstancesPerChunk              = ExecCtx->Internal.MaxInstancesPerChunk;

		ExecCtx->VVMState->NumInstancesExecCached = ExecCtx->NumInstances;
	}

	{
		if (ExecCtx->VVMState->Bytecode == nullptr)
		{
			return;
		}
		FVectorVMBatchState *BatchState = (FVectorVMBatchState *)FMemory::Malloc(ExecCtx->VVMState->BatchOverheadSize + ExecCtx->Internal.PerBatchRegisterDataBytesRequired);
		SetupBatchStatePtrs(ExecCtx, BatchState);

		if (ExecCtx->VVMState->Flags & VVMFlag_HasRandInstruction)
		{
			SetupRandStateForBatch(BatchState);
		}

		int StartInstanceThisChunk = 0;
		int NumChunksThisBatch     = (ExecCtx->NumInstances + ExecCtx->Internal.MaxInstancesPerChunk - 1) / ExecCtx->Internal.MaxInstancesPerChunk;
		for (int ChunkIdxThisBatch = 0; ChunkIdxThisBatch < NumChunksThisBatch; ++ChunkIdxThisBatch, StartInstanceThisChunk += ExecCtx->Internal.MaxInstancesPerChunk)
		{
			int NumInstancesThisChunk = VVM_MIN((int)ExecCtx->Internal.MaxInstancesPerChunk, ExecCtx->NumInstances - StartInstanceThisChunk);
			int NumLoops               = (int)((NumInstancesThisChunk + 3) & ~3) >> 2; //assumes 4-wide ops

			BatchState->ChunkLocalData.ChunkIdx               = ChunkIdxThisBatch;
			BatchState->ChunkLocalData.StartInstanceThisChunk = StartInstanceThisChunk;
			BatchState->ChunkLocalData.NumInstancesThisChunk  = NumInstancesThisChunk;
			
			if (NumLoops == 1)
			{
				ExecChunkSingleLoop(ExecCtx, BatchState);
			}
			else if (NumLoops >= 1)
			{
				ExecChunkMultipleLoops(ExecCtx, BatchState, NumLoops);
			}
		}
	
		if (BatchState->ChunkLocalData.RandCounters)
		{
			FMemory::Free(BatchState->ChunkLocalData.RandCounters);
			BatchState->ChunkLocalData.RandCounters = NULL;
		}
		FMemory::Free(BatchState);
	}

	for (uint32 i = 0; i < ExecCtx->VVMState->MaxOutputDataSet; ++i)
	{
		ExecCtx->DataSets[i].DataSetAccessIndex = ExecCtx->VVMState->NumOutputPerDataSet[i] - 1;
	}
}

FVectorVMState* AllocVectorVMState(TConstArrayView<uint8> ContextData)
{
	if (ContextData.IsEmpty())
	{
		return nullptr;
	}

	FVectorVMRuntimeContext RuntimeContext;

	VectorVM::Bridge::ThawRuntimeContext(ContextData, RuntimeContext);

	//compute the number of overhead bytes for this VVM State
	uint32 ConstBufferOffset = VVM_ALIGN_32(sizeof(FVectorVMState));

	size_t ConstantBufferNumBytes = VVM_PTR_ALIGN(VVM_REG_SIZE * (RuntimeContext.NumConstsRemapped + RuntimeContext.NumNoAdvanceInputs));
	size_t ExtFnTableNumBytes = VVM_PTR_ALIGN(sizeof(FVectorVMExtFunctionData) * (RuntimeContext.MaxExtFnUsed + 1));
	size_t OutputPerDataSetNumBytes = VVM_PTR_ALIGN(sizeof(int32) * RuntimeContext.MaxOutputDataSet);
	size_t ConstMapCacheNumBytes = VVM_PTR_ALIGN((sizeof(uint8) + sizeof(uint16)) * RuntimeContext.NumConstsRemapped);
	size_t InputMapCacheNumBytes = VVM_PTR_ALIGN((sizeof(uint8) + sizeof(uint16)) * RuntimeContext.NumInputsRemapped);
	size_t VVMStateTotalNumBytes = ConstBufferOffset + ConstantBufferNumBytes + ExtFnTableNumBytes + OutputPerDataSetNumBytes + ConstMapCacheNumBytes + InputMapCacheNumBytes;

	uint8* StatePtr = (uint8*)FMemory::Malloc(VVMStateTotalNumBytes, 16);
	if (StatePtr == nullptr)
	{
		return nullptr;
	}

	FVectorVMState* VVMState = (FVectorVMState*)StatePtr;
	VVMState->TotalNumBytes = (uint32)VVMStateTotalNumBytes;

	{ //setup the pointers that are allocated in conjunction with the VVMState
		uint32 ExtFnTableOffset = (uint32)(ConstBufferOffset + ConstantBufferNumBytes);
		uint32 OutputPerDataSetOffset = (uint32)(ExtFnTableOffset + ExtFnTableNumBytes);
		uint32 ConstMapCacheOffset = (uint32)(OutputPerDataSetOffset + OutputPerDataSetNumBytes);
		uint32 InputMapCacheOffset = (uint32)(ConstMapCacheOffset + ConstMapCacheNumBytes);

		VVMState->ConstantBuffers = (FVecReg*)(StatePtr + ConstBufferOffset);
		VVMState->ExtFunctionTable = (FVectorVMExtFunctionData*)(StatePtr + ExtFnTableOffset);
		VVMState->NumOutputPerDataSet = (int32*)(StatePtr + OutputPerDataSetOffset);
		VVMState->ConstMapCacheIdx = (uint8*)(StatePtr + ConstMapCacheOffset);
		VVMState->InputMapCacheIdx = (uint8*)(StatePtr + InputMapCacheOffset);
		VVMState->ConstMapCacheSrc = (uint16*)(VVMState->ConstMapCacheIdx + RuntimeContext.NumConstsRemapped);
		VVMState->InputMapCacheSrc = (uint16*)(VVMState->InputMapCacheIdx + RuntimeContext.NumInputsRemapped);

		check((size_t)((uint8*)VVMState->InputMapCacheSrc - StatePtr) + sizeof(uint16) * RuntimeContext.NumInputsRemapped <= VVMStateTotalNumBytes);

		VVMState->NumInstancesExecCached = 0;
		for (uint32 i = 0; i < RuntimeContext.NumExtFns; ++i)
		{
			VVMState->ExtFunctionTable[i].Function = nullptr;
			VVMState->ExtFunctionTable[i].NumInputs = RuntimeContext.ExtFnTable[i].NumInputs;
			VVMState->ExtFunctionTable[i].NumOutputs = RuntimeContext.ExtFnTable[i].NumOutputs;
		}
	}

	{ //setup the pointers from the optimize context
		VVMState->ConstRemapTable = RuntimeContext.ConstRemap[1];
		VVMState->InputRemapTable = RuntimeContext.InputRemapTable;
		VVMState->InputDataSetOffsets = RuntimeContext.InputDataSetOffsets;
		VVMState->OutputRemapDataSetIdx = RuntimeContext.OutputRemapDataSetIdx;
		VVMState->OutputRemapDataType = RuntimeContext.OutputRemapDataType;
		VVMState->OutputRemapDst = RuntimeContext.OutputRemapDst;

		VVMState->NumTempRegisters = RuntimeContext.NumTempRegisters;
		VVMState->NumConstBuffers = RuntimeContext.NumConstsRemapped;
		VVMState->NumInputBuffers = RuntimeContext.NumInputsRemapped;
		VVMState->NumOutputsRemapped = RuntimeContext.NumOutputsRemapped;
		VVMState->NumOutputBuffers = RuntimeContext.NumOutputInstructions;
		VVMState->Bytecode = RuntimeContext.OutputBytecode;
		VVMState->NumBytecodeBytes = RuntimeContext.NumBytecodeBytes;
		VVMState->NumInputDataSets = RuntimeContext.NumInputDataSets;
		VVMState->MaxOutputDataSet = RuntimeContext.MaxOutputDataSet;
		VVMState->NumDummyRegsRequired = RuntimeContext.NumDummyRegsReq;
		VVMState->Flags = RuntimeContext.Flags & ~VVMFlag_DataMapCacheSetup;
		VVMState->OptimizerHashId = RuntimeContext.HashId;

		VVMState->NumExtFunctions = RuntimeContext.NumExtFns;
		VVMState->MaxExtFnRegisters = RuntimeContext.MaxExtFnRegisters;
	}

	{ //compute fixed batch size
		const size_t NumPtrRegsInTable = VVMState->NumTempRegisters +
			VVMState->NumConstBuffers +
			VVMState->NumInputBuffers * 2 +
			VVMState->NumOutputBuffers;

		const size_t ChunkLocalDataOutputIdxNumBytes = sizeof(uint32) * VVMState->MaxOutputDataSet;
		const size_t ChunkLocalNumOutputNumBytes = sizeof(uint32) * VVMState->MaxOutputDataSet;
		const size_t ChunkLocalOutputMaskIdxNumBytes = sizeof(uint8*) * VVMState->MaxOutputDataSet;
		const size_t ChunkLocalNumExtFnDecodeRegNumBytes = (sizeof(FVecReg*) + sizeof(uint8)) * VVMState->MaxExtFnRegisters + VVM_REG_SIZE * VVMState->NumDummyRegsRequired;
		const size_t RegPtrTableNumBytes = (sizeof(uint32*) + sizeof(uint8)) * NumPtrRegsInTable;
		const size_t BatchOverheadSize = VVM_ALIGN_64(sizeof(FVectorVMBatchState)) +
			ChunkLocalDataOutputIdxNumBytes +
			ChunkLocalNumOutputNumBytes +
			ChunkLocalOutputMaskIdxNumBytes +
			ChunkLocalNumExtFnDecodeRegNumBytes +
			RegPtrTableNumBytes +
			VVM_CHUNK_FIXED_OVERHEAD_SIZE;

		VVMState->BatchOverheadSize = (uint32)BatchOverheadSize;
		VVMState->ChunkLocalDataOutputIdxNumBytes = (uint32)ChunkLocalDataOutputIdxNumBytes;
		VVMState->ChunkLocalNumOutputNumBytes = (uint32)ChunkLocalNumOutputNumBytes;
		VVMState->ChunkLocalOutputMaskIdxNumBytes = (uint32)ChunkLocalOutputMaskIdxNumBytes;
	}

	return VVMState;
}

void FreeVectorVMState(FVectorVMState* VVMState)
{
	if (VVMState != nullptr)
	{
		FMemory::Free(VVMState);
	}
}

#undef VVM_MIN
#undef VVM_MAX
#undef VVM_CLAMP
#undef VVM_ALIGN
#undef VVM_ALIGN_4
#undef VVM_ALIGN_16
#undef VVM_ALIGN_32
#undef VVM_ALIGN_64

#undef VVMSet_m128Const
#undef VVMSet_m128iConst
#undef VVMSet_m128iConst4
#undef VVM_m128Const
#undef VVM_m128iConst


} // VectorVM::Runtime