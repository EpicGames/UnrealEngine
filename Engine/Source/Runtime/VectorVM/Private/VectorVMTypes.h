// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

struct FVectorVMExtFunctionData;

namespace VectorVM
{

union FVecReg
{
	VectorRegister4f v;
	VectorRegister4i i;
};

} // VectorVM

namespace VectorVM::Runtime
{

struct FVectorVMState
{
	uint8* Bytecode;
	uint32                              NumBytecodeBytes;

	FVecReg* ConstantBuffers;        //the last OptimizeCtx->NumNoAdvanceInputs are no advance inputs that are copied in the table setup
	FVectorVMExtFunctionData* ExtFunctionTable;
	int32* NumOutputPerDataSet;

	uint16* ConstRemapTable;
	uint16* InputRemapTable;
	uint16* InputDataSetOffsets;
	uint8* OutputRemapDataSetIdx;
	uint16* OutputRemapDataType;
	uint16* OutputRemapDst;

	uint8* ConstMapCacheIdx;       //these don't get filled out until Exec() is called because they can't be filled out until the state
	uint16* ConstMapCacheSrc;       //of const and input buffers from Niagara is unknown until exec() is called.
	uint8* InputMapCacheIdx;
	uint16* InputMapCacheSrc;
	int32                               NumInstancesExecCached;

	uint32                              Flags;

	uint32                              NumExtFunctions;
	uint32                              MaxExtFnRegisters;

	uint32                              NumTempRegisters;
	uint32                              NumConstBuffers;
	uint32                              NumInputBuffers;
	uint32                              NumInputDataSets;
	uint32                              NumOutputsRemapped;
	uint32                              NumOutputBuffers;
	uint32                              MaxOutputDataSet;
	uint32                              NumDummyRegsRequired;

	//batch stuff
	uint32                              BatchOverheadSize;
	uint32                              ChunkLocalDataOutputIdxNumBytes;
	uint32                              ChunkLocalNumOutputNumBytes;
	uint32                              ChunkLocalOutputMaskIdxNumBytes;

	uint64                              OptimizerHashId;
	uint32                              TotalNumBytes;

	struct
	{
		uint32          NumBytesRequiredPerBatch;
		uint32          PerBatchRegisterDataBytesRequired;
		uint32          MaxChunksPerBatch;
		uint32          MaxInstancesPerChunk;
	} ExecCtxCache;
};

struct FVectorVMRuntimeContext
{
	uint8* OutputBytecode = nullptr;
	uint16* ConstRemap[2] = { nullptr, nullptr };
	uint16* InputRemapTable = nullptr;
	uint16* InputDataSetOffsets = nullptr;
	uint8* OutputRemapDataSetIdx = nullptr;
	uint16* OutputRemapDataType = nullptr;
	uint16* OutputRemapDst = nullptr;

	FVectorVMExtFunctionData* ExtFnTable = nullptr;

	uint32                                NumBytecodeBytes = 0;
	uint32                                MaxOutputDataSet = 0;
	uint16                                NumConstsAlloced = 0;    //upper bound to alloc
	uint32                                NumTempRegisters = 0;
	uint16                                NumConstsRemapped = 0;
	uint16                                NumInputsRemapped = 0;
	uint16                                NumNoAdvanceInputs = 0;
	uint16                                NumInputDataSets = 0;
	uint16                                NumOutputsRemapped = 0;
	uint16                                NumOutputInstructions = 0;
	uint32                                NumExtFns = 0;
	uint32                                MaxExtFnRegisters = 0;
	uint32                                NumDummyRegsReq = 0;     //External function "null" registers
	int32                                 MaxExtFnUsed = 0;
	uint32                                Flags = 0;
	uint64                                HashId = 0;
};

} // VectorVM::Runtime