// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"

#include <atomic>

#include "VectorVM.generated.h"

#if UE_BUILD_DEBUG
#define VM_FORCEINLINE FORCENOINLINE
#else
#define VM_FORCEINLINE FORCEINLINE
#endif

struct FRandomStream;

namespace VectorVM::Runtime
{
	struct FVectorVMExecContext;
	struct FVectorVMRuntimeContext;
	struct FVectorVMState;
}  // VectorVM::Runtime

//TODO: move to a per platform header and have VM scale vectorization according to vector width.
#define VECTOR_WIDTH (128)
#define VECTOR_WIDTH_BYTES (16)
#define VECTOR_WIDTH_FLOATS (4)

UENUM()
enum class EVectorVMBaseTypes : uint8
{
	Float,
	Int,
	Bool,
	Num UMETA(Hidden),
};

UENUM()
enum class EVectorVMOperandLocation : uint8
{
	Register,
	Constant,
	Num
};

//            OpCode                         Category    #in    #out  dispatch,  merge tbl offset, merge tbl count, int/float flags
#define VVM_OP_XM_LIST                                                                                                                                       \
	VVM_OP_XM( done                         , Other     , 0    , 0   , done     , 0             , 0               , VVM_INS_PARAM_FFFFFF) /* 0  */           \
	VVM_OP_XM( add                          , Op        , 2    , 1   , f        , 0             , 2               , VVM_INS_PARAM_FFFFFF) /* 1  */           \
	VVM_OP_XM( sub                          , Op        , 2    , 1   , f        , 2             , 3               , VVM_INS_PARAM_FFFFFF) /* 2  */           \
	VVM_OP_XM( mul                          , Op        , 2    , 1   , f        , 5             , 8               , VVM_INS_PARAM_FFFFFF) /* 3  */           \
	VVM_OP_XM( div                          , Op        , 2    , 1   , f        , 13            , 3               , VVM_INS_PARAM_FFFFFF) /* 4  */           \
	VVM_OP_XM( mad                          , Op        , 3    , 1   , f        , 16            , 7               , VVM_INS_PARAM_FFFFFF) /* 5  */           \
	VVM_OP_XM( lerp                         , Op        , 3    , 1   , f        , 0             , 0               , VVM_INS_PARAM_FFFFFF) /* 6  */           \
	VVM_OP_XM( rcp                          , Op        , 1    , 1   , f        , 0             , 0               , VVM_INS_PARAM_FFFFFF) /* 7  */           \
	VVM_OP_XM( rsq                          , Op        , 1    , 1   , f        , 0             , 0               , VVM_INS_PARAM_FFFFFF) /* 8  */           \
	VVM_OP_XM( sqrt                         , Op        , 1    , 1   , f        , 0             , 0               , VVM_INS_PARAM_FFFFFF) /* 9  */           \
	VVM_OP_XM( neg                          , Op        , 1    , 1   , f        , 23            , 1               , VVM_INS_PARAM_FFFFFF) /* 10 */           \
	VVM_OP_XM( abs                          , Op        , 1    , 1   , f        , 0             , 0               , VVM_INS_PARAM_FFFFFF) /* 11 */           \
	VVM_OP_XM( exp                          , Op        , 1    , 1   , f        , 0             , 0               , VVM_INS_PARAM_FFFFFF) /* 12 */           \
	VVM_OP_XM( exp2                         , Op        , 1    , 1   , f        , 0             , 0               , VVM_INS_PARAM_FFFFFF) /* 13 */           \
	VVM_OP_XM( log                          , Op        , 1    , 1   , f        , 0             , 0               , VVM_INS_PARAM_FFFFFF) /* 14 */           \
	VVM_OP_XM( log2                         , Op        , 1    , 1   , f        , 0             , 0               , VVM_INS_PARAM_FFFFFF) /* 15 */           \
	VVM_OP_XM( sin                          , Op        , 1    , 1   , f        , 24            , 1               , VVM_INS_PARAM_FFFFFF) /* 16 */           \
	VVM_OP_XM( cos                          , Op        , 1    , 1   , f        , 25            , 1               , VVM_INS_PARAM_FFFFFF) /* 17 */           \
	VVM_OP_XM( tan                          , Op        , 1    , 1   , f        , 0             , 0               , VVM_INS_PARAM_FFFFFF) /* 18 */           \
	VVM_OP_XM( asin                         , Op        , 1    , 1   , f        , 0             , 0               , VVM_INS_PARAM_FFFFFF) /* 19 */           \
	VVM_OP_XM( acos                         , Op        , 1    , 1   , f        , 0             , 0               , VVM_INS_PARAM_FFFFFF) /* 20 */           \
	VVM_OP_XM( atan                         , Op        , 1    , 1   , f        , 0             , 0               , VVM_INS_PARAM_FFFFFF) /* 21 */           \
	VVM_OP_XM( atan2                        , Op        , 2    , 1   , f        , 0             , 0               , VVM_INS_PARAM_FFFFFF) /* 22 */           \
	VVM_OP_XM( ceil                         , Op        , 1    , 1   , f        , 0             , 0               , VVM_INS_PARAM_FFFFFF) /* 23 */           \
	VVM_OP_XM( floor                        , Op        , 1    , 1   , f        , 0             , 0               , VVM_INS_PARAM_FFFFFF) /* 24 */           \
	VVM_OP_XM( fmod                         , Op        , 2    , 1   , f        , 26            , 1               , VVM_INS_PARAM_FFFFFF) /* 25 */           \
	VVM_OP_XM( frac                         , Op        , 1    , 1   , f        , 0             , 0               , VVM_INS_PARAM_FFFFFF) /* 26 */           \
	VVM_OP_XM( trunc                        , Op        , 1    , 1   , f        , 0             , 0               , VVM_INS_PARAM_FFFFFF) /* 27 */           \
	VVM_OP_XM( clamp                        , Op        , 3    , 1   , f        , 0             , 0               , VVM_INS_PARAM_FFFFFF) /* 28 */           \
	VVM_OP_XM( min                          , Op        , 2    , 1   , f        , 0             , 0               , VVM_INS_PARAM_FFFFFF) /* 29 */           \
	VVM_OP_XM( max                          , Op        , 2    , 1   , f        , 27            , 1               , VVM_INS_PARAM_FFFFFF) /* 30 */           \
	VVM_OP_XM( pow                          , Op        , 2    , 1   , f        , 0             , 0               , VVM_INS_PARAM_FFFFFF) /* 31 */           \
	VVM_OP_XM( round                        , Op        , 1    , 1   , f        , 0             , 0               , VVM_INS_PARAM_FFFFFF) /* 32 */           \
	VVM_OP_XM( sign                         , Op        , 1    , 1   , f        , 0             , 0               , VVM_INS_PARAM_FFFFFF) /* 33 */           \
	VVM_OP_XM( step                         , Op        , 2    , 1   , f        , 0             , 0               , VVM_INS_PARAM_FFFFFF) /* 34 */           \
	VVM_OP_XM( random                       , Op        , 1    , 1   , null     , 28            , 2               , VVM_INS_PARAM_FFFFFF) /* 35 */           \
	VVM_OP_XM( noise                        , Op        , 1    , 1   , f        , 0             , 0               , VVM_INS_PARAM_FFFFFF) /* 36 */           \
	VVM_OP_XM( cmplt                        , Op        , 2    , 1   , f        , 30            , 3               , VVM_INS_PARAM_FFFIFF) /* 37 */           \
	VVM_OP_XM( cmple                        , Op        , 2    , 1   , f        , 33            , 3               , VVM_INS_PARAM_FFFIFF) /* 38 */           \
	VVM_OP_XM( cmpgt                        , Op        , 2    , 1   , f        , 36            , 2               , VVM_INS_PARAM_FFFIFF) /* 39 */           \
	VVM_OP_XM( cmpge                        , Op        , 2    , 1   , f        , 38            , 2               , VVM_INS_PARAM_FFFIFF) /* 40 */           \
	VVM_OP_XM( cmpeq                        , Op        , 2    , 1   , f        , 40            , 3               , VVM_INS_PARAM_FFFIFF) /* 41 */           \
	VVM_OP_XM( cmpneq                       , Op        , 2    , 1   , f        , 43            , 2               , VVM_INS_PARAM_FFFIFF) /* 42 */           \
	VVM_OP_XM( select                       , Op        , 3    , 1   , f        , 45            , 2               , VVM_INS_PARAM_FFFFFI) /* 43 */           \
	VVM_OP_XM( addi                         , Op        , 2    , 1   , i        , 47            , 2               , VVM_INS_PARAM_FFIIII) /* 44 */           \
	VVM_OP_XM( subi                         , Op        , 2    , 1   , i        , 0             , 0               , VVM_INS_PARAM_FFIIII) /* 45 */           \
	VVM_OP_XM( muli                         , Op        , 2    , 1   , i        , 49            , 1               , VVM_INS_PARAM_FFIIII) /* 46 */           \
	VVM_OP_XM( divi                         , Op        , 2    , 1   , i        , 0             , 0               , VVM_INS_PARAM_FFIIII) /* 47 */           \
	VVM_OP_XM( clampi                       , Op        , 3    , 1   , i        , 0             , 0               , VVM_INS_PARAM_FFIIII) /* 48 */           \
	VVM_OP_XM( mini                         , Op        , 2    , 1   , i        , 0             , 0               , VVM_INS_PARAM_FFIIII) /* 49 */           \
	VVM_OP_XM( maxi                         , Op        , 2    , 1   , i        , 0             , 0               , VVM_INS_PARAM_FFIIII) /* 50 */           \
	VVM_OP_XM( absi                         , Op        , 1    , 1   , i        , 0             , 0               , VVM_INS_PARAM_FFIIII) /* 51 */           \
	VVM_OP_XM( negi                         , Op        , 1    , 1   , i        , 0             , 0               , VVM_INS_PARAM_FFIIII) /* 52 */           \
	VVM_OP_XM( signi                        , Op        , 1    , 1   , i        , 0             , 0               , VVM_INS_PARAM_FFIIII) /* 53 */           \
	VVM_OP_XM( randomi                      , Op        , 1    , 1   , null     , 0             , 0               , VVM_INS_PARAM_FFIIII) /* 54 */           \
	VVM_OP_XM( cmplti                       , Op        , 2    , 1   , i        , 50            , 3               , VVM_INS_PARAM_FFIIII) /* 55 */           \
	VVM_OP_XM( cmplei                       , Op        , 2    , 1   , i        , 53            , 3               , VVM_INS_PARAM_FFIIII) /* 56 */           \
	VVM_OP_XM( cmpgti                       , Op        , 2    , 1   , i        , 59            , 2               , VVM_INS_PARAM_FFIIII) /* 57 */           \
	VVM_OP_XM( cmpgei                       , Op        , 2    , 1   , i        , 61            , 2               , VVM_INS_PARAM_FFIIII) /* 58 */           \
	VVM_OP_XM( cmpeqi                       , Op        , 2    , 1   , i        , 56            , 3               , VVM_INS_PARAM_FFIIII) /* 59 */           \
	VVM_OP_XM( cmpneqi                      , Op        , 2    , 1   , i        , 63            , 2               , VVM_INS_PARAM_FFIIII) /* 60 */           \
	VVM_OP_XM( bit_and                      , Op        , 2    , 1   , i        , 65            , 1               , VVM_INS_PARAM_FFIIII) /* 61 */           \
	VVM_OP_XM( bit_or                       , Op        , 2    , 1   , i        , 66            , 1               , VVM_INS_PARAM_FFIIII) /* 62 */           \
	VVM_OP_XM( bit_xor                      , Op        , 2    , 1   , i        , 0             , 0               , VVM_INS_PARAM_FFIIII) /* 63 */           \
	VVM_OP_XM( bit_not                      , Op        , 1    , 1   , i        , 0             , 0               , VVM_INS_PARAM_FFIIII) /* 64 */           \
	VVM_OP_XM( bit_lshift                   , Op        , 2    , 1   , i        , 67            , 1               , VVM_INS_PARAM_FFIIII) /* 65 */           \
	VVM_OP_XM( bit_rshift                   , Op        , 2    , 1   , i        , 68            , 1               , VVM_INS_PARAM_FFIIII) /* 66 */           \
	VVM_OP_XM( logic_and                    , Op        , 2    , 1   , i        , 0             , 0               , VVM_INS_PARAM_FFIIII) /* 67 */           \
	VVM_OP_XM( logic_or                     , Op        , 2    , 1   , i        , 0             , 0               , VVM_INS_PARAM_FFIIII) /* 68 */           \
	VVM_OP_XM( logic_xor                    , Op        , 2    , 1   , i        , 0             , 0               , VVM_INS_PARAM_FFIIII) /* 69 */           \
	VVM_OP_XM( logic_not                    , Op        , 1    , 1   , i        , 0             , 0               , VVM_INS_PARAM_FFIIII) /* 70 */           \
	VVM_OP_XM( f2i                          , Op        , 1    , 1   , i        , 74            , 3               , VVM_INS_PARAM_FFFFFI) /* 71 */           \
	VVM_OP_XM( i2f                          , Op        , 1    , 1   , f        , 69            , 5               , VVM_INS_PARAM_FFFFIF) /* 72 */           \
	VVM_OP_XM( f2b                          , Op        , 1    , 1   , f        , 0             , 0               , VVM_INS_PARAM_FFFFFI) /* 73 */           \
	VVM_OP_XM( b2f                          , Op        , 1    , 1   , f        , 0             , 0               , VVM_INS_PARAM_FFFFIF) /* 74 */           \
	VVM_OP_XM( i2b                          , Op        , 1    , 1   , i        , 0             , 0               , VVM_INS_PARAM_FFFFII) /* 75 */           \
	VVM_OP_XM( b2i                          , Op        , 1    , 1   , i        , 77            , 1               , VVM_INS_PARAM_FFFFII) /* 76 */           \
	VVM_OP_XM( inputdata_float              , Input     , 0    , 0   , null     , 0             , 0               , VVM_INS_PARAM_FFFFFF) /* 77 */           \
	VVM_OP_XM( inputdata_int32              , Input     , 0    , 0   , null     , 0             , 0               , VVM_INS_PARAM_FFFFFF) /* 78 */           \
	VVM_OP_XM( inputdata_half               , Input     , 0    , 0   , null     , 0             , 0               , VVM_INS_PARAM_FFFFFF) /* 79 */           \
	VVM_OP_XM( inputdata_noadvance_float    , Input     , 0    , 0   , null     , 0             , 0               , VVM_INS_PARAM_FFFFFF) /* 80 */           \
	VVM_OP_XM( inputdata_noadvance_int32    , Input     , 0    , 0   , null     , 0             , 0               , VVM_INS_PARAM_FFFFFF) /* 81 */           \
	VVM_OP_XM( inputdata_noadvance_half     , Input     , 0    , 0   , null     , 0             , 0               , VVM_INS_PARAM_FFFFFF) /* 82 */           \
	VVM_OP_XM( outputdata_float             , Output    , 0    , 0   , null     , 0             , 0               , VVM_INS_PARAM_FFFFFI) /* 83 */           \
	VVM_OP_XM( outputdata_int32             , Output    , 0    , 0   , null     , 0             , 0               , VVM_INS_PARAM_FFFIII) /* 84 */           \
	VVM_OP_XM( outputdata_half              , Output    , 0    , 0   , null     , 0             , 0               , VVM_INS_PARAM_FFFFFI) /* 85 */           \
	VVM_OP_XM( acquireindex                 , IndexGen  , 1    , 0   , null     , 0             , 0               , VVM_INS_PARAM_FFFFII) /* 86 */           \
	VVM_OP_XM( external_func_call           , ExtFnCall , 0    , 0   , null     , 0             , 0               , VVM_INS_PARAM_FFFFFF) /* 87 */           \
	VVM_OP_XM( exec_index                   , Op        , 1    , 1   , null     , 78            , 2               , VVM_INS_PARAM_FFFFFI) /* 88 */           \
	VVM_OP_XM( noise2D                      , Op		, 2    , 1   , f        , 0             , 0               , VVM_INS_PARAM_FFFFFF) /* 89 */           \
	VVM_OP_XM( noise3D                      , Op		, 3    , 1   , f        , 0             , 0               , VVM_INS_PARAM_FFFFFF) /* 90 */           \
	VVM_OP_XM( enter_stat_scope             , Stat      , 1    , 0   , null     , 0             , 0               , VVM_INS_PARAM_FFFFFF) /* 91 */           \
	VVM_OP_XM( exit_stat_scope              , Stat      , 0    , 0   , null     , 0             , 0               , VVM_INS_PARAM_FFFFFF) /* 92 */           \
	VVM_OP_XM( update_id                    , RWBuffer  , 0    , 0   , null     , 0             , 0               , VVM_INS_PARAM_FFFFFF) /* 93 */           \
	VVM_OP_XM( acquire_id                   , RWBuffer  , 1    , 0   , null     , 0             , 0               , VVM_INS_PARAM_FFFFFF) /* 94 */           \
	VVM_OP_XM( half_to_float                , Op        , 1    , 1   , null     , 0             , 0               , VVM_INS_PARAM_FFFFFF) /* 95 */           \
	VVM_OP_XM( fasi                         , Op        , 1    , 1   , null     , 0             , 0               , VVM_INS_PARAM_FFFFFF) /* 96 */           \
	VVM_OP_XM( iasf                         , Op        , 1    , 1   , null     , 0             , 0               , VVM_INS_PARAM_FFFFFI) /* 97 */           \
	/* Merged ops -- combined ops that show up frequently together in Fornite.  There are three types: */                                                    \
	/* 1. exec_index that get immediately fed into an add or i2f                                       */                                                    \
	/* 2. ops with identical inputs                                                                    */                                                    \
	/* 3. ops where the output of one chain to the next.. ie a mul that feeds directly into a sub      */                                                    \
	VVM_OP_XM( exec_indexf                  , Op        , 1    , 1   , null     , 0             , 0               , VVM_INS_PARAM_FFIIIF) /* 98 */           \
	VVM_OP_XM( exec_index_addi              , Op        , 1    , 1   , null     , 0             , 0               , VVM_INS_PARAM_FFIIII) /* 99 */           \
	VVM_OP_XM( cmplt_select                 , Op        , 4    , 1   , f        , 0             , 0               , VVM_INS_PARAM_FFFFFF) /* 100 */           \
	VVM_OP_XM( cmple_select                 , Op        , 4    , 1   , f        , 0             , 0               , VVM_INS_PARAM_FFFFFF) /* 101 */           \
	VVM_OP_XM( cmpeq_select                 , Op        , 4    , 1   , f        , 0             , 0               , VVM_INS_PARAM_FFFFFF) /* 102 */          \
	VVM_OP_XM( cmplti_select                , Op        , 4    , 1   , f        , 0             , 0               , VVM_INS_PARAM_FIIFFF) /* 103 */          \
	VVM_OP_XM( cmplei_select                , Op        , 4    , 1   , f        , 0             , 0               , VVM_INS_PARAM_FIIFFF) /* 104 */          \
	VVM_OP_XM( cmpeqi_select                , Op        , 4    , 1   , f        , 0             , 0               , VVM_INS_PARAM_FIIFFF) /* 105 */          \
	VVM_OP_XM( cmplt_logic_and              , Op        , 3    , 1   , f        , 0             , 0               , VVM_INS_PARAM_FFFFII) /* 106 */          \
	VVM_OP_XM( cmple_logic_and              , Op        , 3    , 1   , f        , 0             , 0               , VVM_INS_PARAM_FFFFII) /* 107 */          \
	VVM_OP_XM( cmpgt_logic_and              , Op        , 3    , 1   , f        , 0             , 0               , VVM_INS_PARAM_FFFFII) /* 108 */          \
	VVM_OP_XM( cmpge_logic_and              , Op        , 3    , 1   , f        , 0             , 0               , VVM_INS_PARAM_FFFFII) /* 109 */          \
	VVM_OP_XM( cmpeq_logic_and              , Op        , 3    , 1   , f        , 0             , 0               , VVM_INS_PARAM_FFFFII) /* 110 */          \
	VVM_OP_XM( cmpne_logic_and              , Op        , 3    , 1   , f        , 0             , 0               , VVM_INS_PARAM_FFFFII) /* 111 */          \
	VVM_OP_XM( cmplti_logic_and             , Op        , 3    , 1   , i        , 0             , 0               , VVM_INS_PARAM_FIIFII) /* 112 */          \
	VVM_OP_XM( cmplei_logic_and             , Op        , 3    , 1   , i        , 0             , 0               , VVM_INS_PARAM_FIIFII) /* 113 */          \
	VVM_OP_XM( cmpgti_logic_and             , Op        , 3    , 1   , i        , 0             , 0               , VVM_INS_PARAM_FIIFII) /* 114 */          \
	VVM_OP_XM( cmpgei_logic_and             , Op        , 3    , 1   , i        , 0             , 0               , VVM_INS_PARAM_FIIFIF) /* 115 */          \
	VVM_OP_XM( cmpeqi_logic_and             , Op        , 3    , 1   , i        , 0             , 0               , VVM_INS_PARAM_FIIFIF) /* 116 */          \
	VVM_OP_XM( cmpnei_logic_and             , Op        , 3    , 1   , i        , 0             , 0               , VVM_INS_PARAM_FIIFIF) /* 117 */          \
	VVM_OP_XM( cmplt_logic_or               , Op        , 3    , 1   , f        , 0             , 0               , VVM_INS_PARAM_FFFFFF) /* 118 */          \
	VVM_OP_XM( cmple_logic_or               , Op        , 3    , 1   , f        , 0             , 0               , VVM_INS_PARAM_FFFFFF) /* 119 */          \
	VVM_OP_XM( cmpgt_logic_or               , Op        , 3    , 1   , f        , 0             , 0               , VVM_INS_PARAM_FFFFFF) /* 120 */          \
	VVM_OP_XM( cmpge_logic_or               , Op        , 3    , 1   , f        , 0             , 0               , VVM_INS_PARAM_FFFFFF) /* 121 */          \
	VVM_OP_XM( cmpeq_logic_or               , Op        , 3    , 1   , f        , 0             , 0               , VVM_INS_PARAM_FFFFFF) /* 122 */          \
	VVM_OP_XM( cmpne_logic_or               , Op        , 3    , 1   , f        , 0             , 0               , VVM_INS_PARAM_FFFFFF) /* 123 */          \
	VVM_OP_XM( cmplti_logic_or              , Op        , 3    , 1   , i        , 0             , 0               , VVM_INS_PARAM_FIIFFF) /* 124 */          \
	VVM_OP_XM( cmplei_logic_or              , Op        , 3    , 1   , i        , 0             , 0               , VVM_INS_PARAM_FIIFFF) /* 125 */          \
	VVM_OP_XM( cmpgti_logic_or              , Op        , 3    , 1   , i        , 0             , 0               , VVM_INS_PARAM_FIIFFF) /* 126 */          \
	VVM_OP_XM( cmpgei_logic_or              , Op        , 3    , 1   , i        , 0             , 0               , VVM_INS_PARAM_FIIFFF) /* 127 */          \
	VVM_OP_XM( cmpeqi_logic_or              , Op        , 3    , 1   , i        , 0             , 0               , VVM_INS_PARAM_FIIFFF) /* 128 */          \
	VVM_OP_XM( cmpnei_logic_or              , Op        , 3    , 1   , i        , 0             , 0               , VVM_INS_PARAM_FIIFFF) /* 129 */          \
	VVM_OP_XM( mad_add                      , Op        , 4    , 1   , f        , 0             , 0               , VVM_INS_PARAM_FFFFFF) /* 130 */          \
	VVM_OP_XM( mad_sub0                     , Op        , 4    , 1   , f        , 0             , 0               , VVM_INS_PARAM_FFFFFF) /* 131 */          \
	VVM_OP_XM( mad_sub1                     , Op        , 4    , 1   , f        , 0             , 0               , VVM_INS_PARAM_FFFFFF) /* 132 */          \
	VVM_OP_XM( mad_mul                      , Op        , 4    , 1   , f        , 0             , 0               , VVM_INS_PARAM_FFFFFF) /* 133 */          \
	VVM_OP_XM( mad_sqrt                     , Op        , 3    , 1   , f        , 0             , 0               , VVM_INS_PARAM_FFFFFF) /* 134 */          \
	VVM_OP_XM( mad_mad0                     , Op        , 5    , 1   , f        , 0             , 0               , VVM_INS_PARAM_FFFFFF) /* 135 */          \
	VVM_OP_XM( mad_mad1                     , Op        , 5    , 1   , f        , 0             , 0               , VVM_INS_PARAM_FFFFFF) /* 136 */          \
	VVM_OP_XM( mul_mad0                     , Op        , 4    , 1   , f        , 0             , 0               , VVM_INS_PARAM_FFFFFF) /* 137 */          \
	VVM_OP_XM( mul_mad1                     , Op        , 4    , 1   , f        , 0             , 0               , VVM_INS_PARAM_FFFFFF) /* 138 */          \
	VVM_OP_XM( mul_add                      , Op        , 3    , 1   , f        , 0             , 0               , VVM_INS_PARAM_FFFFFF) /* 139 */          \
	VVM_OP_XM( mul_sub0                     , Op        , 3    , 1   , f        , 0             , 0               , VVM_INS_PARAM_FFFFFF) /* 140 */          \
	VVM_OP_XM( mul_sub1                     , Op        , 3    , 1   , f        , 0             , 0               , VVM_INS_PARAM_FFFFFF) /* 141 */          \
	VVM_OP_XM( mul_mul                      , Op        , 3    , 1   , f        , 0             , 0               , VVM_INS_PARAM_FFFFFF) /* 142 */          \
	VVM_OP_XM( mul_max                      , Op        , 3    , 1   , f        , 0             , 0               , VVM_INS_PARAM_FFFFFF) /* 143 */          \
	VVM_OP_XM( mul_2x                       , Op        , 2    , 2   , f        , 0             , 0               , VVM_INS_PARAM_FFFFFF) /* 144 */          \
	VVM_OP_XM( add_mad1                     , Op        , 4    , 1   , f        , 0             , 0               , VVM_INS_PARAM_FFFFFF) /* 145 */          \
	VVM_OP_XM( add_add                      , Op        , 3    , 1   , f        , 0             , 0               , VVM_INS_PARAM_FFFFFF) /* 146 */          \
	VVM_OP_XM( sub_cmplt1                   , Op        , 3    , 1   , f        , 0             , 0               , VVM_INS_PARAM_FFIFFF) /* 147 */          \
	VVM_OP_XM( sub_neg                      , Op        , 2    , 1   , f        , 0             , 0               , VVM_INS_PARAM_FFFFFF) /* 148 */          \
	VVM_OP_XM( sub_mul                      , Op        , 3    , 1   , f        , 0             , 0               , VVM_INS_PARAM_FFFFFF) /* 149 */          \
	VVM_OP_XM( div_mad0                     , Op        , 4    , 1   , f        , 0             , 0               , VVM_INS_PARAM_FFFFFF) /* 150 */          \
	VVM_OP_XM( div_f2i                      , Op        , 2    , 1   , i        , 0             , 0               , VVM_INS_PARAM_FFFFFI) /* 151 */          \
	VVM_OP_XM( div_mul                      , Op        , 3    , 1   , f        , 0             , 0               , VVM_INS_PARAM_FFFFFF) /* 152 */          \
	VVM_OP_XM( muli_addi                    , Op        , 3    , 1   , i        , 0             , 0               , VVM_INS_PARAM_IIIIII) /* 153 */          \
	VVM_OP_XM( addi_bit_rshift				, Op        , 3    , 1   , i        , 0             , 0               , VVM_INS_PARAM_IIIIII) /* 154 */          \
	VVM_OP_XM( addi_muli                    , Op        , 3    , 1   , i        , 0             , 0               , VVM_INS_PARAM_IIIIII) /* 155 */          \
	VVM_OP_XM( b2i_2x                       , Op        , 1    , 2   , i        , 0             , 0               , VVM_INS_PARAM_IIIIII) /* 156 */          \
	VVM_OP_XM( i2f_div0                     , Op        , 2    , 1   , f        , 0             , 0               , VVM_INS_PARAM_FFFFFF) /* 157 */          \
	VVM_OP_XM( i2f_div1                     , Op        , 2    , 1   , f        , 0             , 0               , VVM_INS_PARAM_FFFFFF) /* 158 */          \
	VVM_OP_XM( i2f_mul                      , Op        , 2    , 1   , f        , 0             , 0               , VVM_INS_PARAM_FFFFFF) /* 159 */          \
	VVM_OP_XM( i2f_mad0                     , Op        , 3    , 1   , f        , 0             , 0               , VVM_INS_PARAM_FFFFFF) /* 160 */          \
	VVM_OP_XM( i2f_mad1                     , Op        , 3    , 1   , f        , 0             , 0               , VVM_INS_PARAM_FFFFFF) /* 161 */          \
	VVM_OP_XM( f2i_select1                  , Op        , 3    , 1   , i        , 0             , 0               , VVM_INS_PARAM_IIFIII) /* 162 */          \
	VVM_OP_XM( f2i_maxi                     , Op        , 2    , 1   , i        , 0             , 0               , VVM_INS_PARAM_IIIFII) /* 163 */          \
	VVM_OP_XM( f2i_addi                     , Op        , 2    , 1   , i        , 0             , 0               , VVM_INS_PARAM_IIIFII) /* 164 */          \
	VVM_OP_XM( fmod_add                     , Op        , 3    , 1   , f        , 0             , 0               , VVM_INS_PARAM_FFFFFF) /* 165 */          \
	VVM_OP_XM( bit_and_i2f                  , Op        , 2    , 1   , f        , 0             , 0               , VVM_INS_PARAM_FFFFFF) /* 166 */          \
	VVM_OP_XM( bit_rshift_bit_and           , Op        , 3    , 1   , i        , 0             , 0               , VVM_INS_PARAM_FFFFFF) /* 167 */          \
	VVM_OP_XM( neg_cmplt                    , Op        , 2    , 1   , f        , 0             , 0               , VVM_INS_PARAM_FFFFFF) /* 168 */          \
	VVM_OP_XM( bit_or_muli                  , Op        , 3    , 1   , i        , 0             , 0               , VVM_INS_PARAM_FFFFFF) /* 169 */          \
	VVM_OP_XM( bit_lshift_bit_or            , Op        , 3    , 1   , i        , 0             , 0               , VVM_INS_PARAM_FFFFFF) /* 170 */          \
	VVM_OP_XM( random_add                   , Op        , 2    , 1   , null     , 0             , 0               , VVM_INS_PARAM_FFFFFF) /* 171 */          \
	VVM_OP_XM( random_2x                    , Op        , 1    , 2   , null     , 0             , 0               , VVM_INS_PARAM_FFFFFF) /* 172 */          \
	VVM_OP_XM( max_f2i                      , Op        , 2    , 1   , i        , 0             , 0               , VVM_INS_PARAM_FFFFFF) /* 173 */          \
	VVM_OP_XM( select_mul                   , Op        , 4    , 1   , f        , 0             , 0               , VVM_INS_PARAM_FFFFFF) /* 174 */          \
	VVM_OP_XM( select_add                   , Op        , 4    , 1   , f        , 0             , 0               , VVM_INS_PARAM_FFFFFF) /* 175 */          \
	VVM_OP_XM( sin_cos                      , Op        , 1    , 2   , null     , 0             , 0               , VVM_INS_PARAM_FFFFFF) /* 176 */          \
	VVM_OP_XM( outputdata_float_from_half   , Output    , 0    , 0   , null     , 0             , 0               , VVM_INS_PARAM_FFFFFI) /* 177 */          \
	VVM_OP_XM( outputdata_half_from_half    , Output    , 0    , 0   , null     , 0             , 0               , VVM_INS_PARAM_FFFFFI) /* 178 */

#define VVM_OP_XM(n, ...) n,
enum class EVectorVMOp : uint8
{
	VVM_OP_XM_LIST
	NumOpcodes
};
#undef VVM_OP_XM

//TODO: 
//All of this stuff can be handled by the VM compiler rather than dirtying the VM code.
//Some require RWBuffer like support.
struct FDataSetMeta
{
	using FInputRegisterView = TArrayView<uint8 const* RESTRICT const>;
	using FOutputRegisterView = TArrayView<uint8* RESTRICT const>;

	FInputRegisterView InputRegisters;
	FOutputRegisterView OutputRegisters;

	uint32 InputRegisterTypeOffsets[3];
	uint32 OutputRegisterTypeOffsets[3];

	int32 DataSetAccessIndex = INDEX_NONE;	// index for individual elements of this set

	int32 InstanceOffset = INDEX_NONE;		// offset of the first instance processed 
	
	TArray<int32>* RESTRICT IDTable = nullptr;
	TArray<int32>* RESTRICT FreeIDTable = nullptr;
	TArray<int32>* RESTRICT SpawnedIDsTable = nullptr;

	/** Number of free IDs in the FreeIDTable */
	int32* NumFreeIDs = nullptr;

	/** MaxID used in this execution. */
	int32* MaxUsedID = nullptr;

	int32 *NumSpawnedIDs = nullptr;

	int32 IDAcquireTag = INDEX_NONE;

	FDataSetMeta() = default;

	inline void Reset()
	{
		InputRegisters = FInputRegisterView();
		OutputRegisters = FOutputRegisterView();
		DataSetAccessIndex = INDEX_NONE;
		InstanceOffset = INDEX_NONE;
		IDTable = nullptr;
		FreeIDTable = nullptr;
		SpawnedIDsTable = nullptr;
		NumFreeIDs = nullptr;
		MaxUsedID = nullptr;
		IDAcquireTag = INDEX_NONE;
	}

	inline void Init(FInputRegisterView InInputRegisters, FOutputRegisterView InOutputRegisters, int32 InInstanceOffset, TArray<int32>* InIDTable, TArray<int32>* InFreeIDTable, int32* InNumFreeIDs, int32 *InNumSpawnedIDs, int32* InMaxUsedID, int32 InIDAcquireTag, TArray<int32>* InSpawnedIDsTable)
	{
		InputRegisters = InInputRegisters;
		OutputRegisters = InOutputRegisters;

		DataSetAccessIndex = INDEX_NONE;
		InstanceOffset = InInstanceOffset;
		IDTable = InIDTable;
		FreeIDTable = InFreeIDTable;
		NumFreeIDs = InNumFreeIDs;
		NumSpawnedIDs = InNumSpawnedIDs;
		MaxUsedID = InMaxUsedID;
		IDAcquireTag = InIDAcquireTag;
		SpawnedIDsTable = InSpawnedIDsTable;
	}

private:
	// Non-copyable and non-movable
	FDataSetMeta(FDataSetMeta&&) = delete;
	FDataSetMeta(const FDataSetMeta&) = delete;
	FDataSetMeta& operator=(FDataSetMeta&&) = delete;
	FDataSetMeta& operator=(const FDataSetMeta&) = delete;
};

class FVectorVMExternalFunctionContext
{
public:
	MS_ALIGN(32) uint32** RegisterData GCC_ALIGN(32);
	uint8* RegInc;

	int                      RegReadCount;
	int                      NumRegisters;

	int                      StartInstance;
	int                      NumInstances;
	int                      NumLoops;
	int                      PerInstanceFnInstanceIdx;

	void** UserPtrTable;
	int                      NumUserPtrs;

	FRandomStream* RandStream;
	int32** RandCounters;
	TArrayView<FDataSetMeta> DataSets;

	VM_FORCEINLINE int32                                  GetStartInstance() const { return StartInstance; }
	VM_FORCEINLINE int32                                  GetNumInstances() const { return NumInstances; }
	inline int32* GetRandCounters()
	{
		if (!*RandCounters)
		{
			*RandCounters = (int*)FMemory::MallocZeroed(sizeof(int32) * NumInstances);
		}
		return *RandCounters;
	}
	inline FRandomStream& GetRandStream() { return *RandStream; }
	inline void* GetUserPtrTable(int32 UserPtrIdx) { check(UserPtrIdx < NumUserPtrs);  return UserPtrTable[UserPtrIdx]; }
	template<uint32 InstancesPerOp> inline int32  GetNumLoops() const { static_assert(InstancesPerOp == 4); return NumLoops; };

	inline float* GetNextRegister(int32* OutAdvanceOffset)
	{
		check(RegReadCount < NumRegisters);
		*OutAdvanceOffset = RegInc[RegReadCount];
		return (float*)RegisterData[RegReadCount++];
	}
};

DECLARE_DELEGATE_OneParam(FVMExternalFunction, FVectorVMExternalFunctionContext& /*Context*/);

struct FVectorVMExtFunctionData
{
	const FVMExternalFunction* Function;
	int32                      NumInputs;
	int32                      NumOutputs;
};

namespace VectorVM
{
	/** Get total number of op-codes */
	VECTORVM_API uint8 GetNumOpCodes();

#if WITH_EDITOR
	VECTORVM_API FString GetOpName(EVectorVMOp Op);
	VECTORVM_API FString GetOperandLocationName(EVectorVMOperandLocation Location);
#endif

	VECTORVM_API uint8 CreateSrcOperandMask(EVectorVMOperandLocation Type0, EVectorVMOperandLocation Type1 = EVectorVMOperandLocation::Register, EVectorVMOperandLocation Type2 = EVectorVMOperandLocation::Register);

#if STATS

	struct FStatScopeData
	{
		TStatId StatId;
		std::atomic<uint64> ExecutionCycleCount = { 0 };

		FStatScopeData(TStatId InStatId)
			: StatId(InStatId)
		{
		}

		FStatScopeData(const FStatScopeData& InObj)
			: StatId(InObj.StatId)
		{
			ExecutionCycleCount.store(InObj.ExecutionCycleCount.load());
		}
	};

#endif // STATS

	VECTORVM_API void Init();

	#define VVM_EXT_FUNC_INPUT_LOC_BIT (unsigned short)(1<<15)
	#define VVM_EXT_FUNC_INPUT_LOC_MASK (unsigned short)~VVM_EXT_FUNC_INPUT_LOC_BIT

	template<typename T>
	struct FUserPtrHandler
	{
		int32 UserPtrIdx;
		T* Ptr;
		FUserPtrHandler(FVectorVMExternalFunctionContext& Context)
		{
			int32 AdvanceOffset;
			int32* ConstPtr = (int32*)Context.GetNextRegister(&AdvanceOffset);
			check(AdvanceOffset == 0); //must be constant
			UserPtrIdx = *ConstPtr;
			check(UserPtrIdx != INDEX_NONE);
			Ptr = (T*)Context.GetUserPtrTable(UserPtrIdx);
		}

		VM_FORCEINLINE T* Get() { return Ptr; }
		VM_FORCEINLINE T* operator->() { return Ptr; }
		VM_FORCEINLINE operator T*() { return Ptr; }

		VM_FORCEINLINE const T* Get() const { return Ptr; }
		VM_FORCEINLINE const T* operator->() const { return Ptr; }
		VM_FORCEINLINE operator const T* () const { return Ptr; }
	};

	// A flexible handler that can deal with either constant or register inputs.
	template<typename T>
	struct FExternalFuncInputHandler
	{
	private:
		/** Either byte offset into constant table or offset into register table deepening on VVM_INPUT_LOCATION_BIT */
		const T* RESTRICT InputPtr = nullptr;
		const T* RESTRICT StartPtr = nullptr;
		int32 AdvanceOffset = 0;

	public:
		FExternalFuncInputHandler() = default;

		inline FExternalFuncInputHandler(FVectorVMExternalFunctionContext& Context)
		{
			Init(Context);
		}

		void Init(FVectorVMExternalFunctionContext& Context)
		{
			InputPtr = (T*)Context.GetNextRegister(&AdvanceOffset);
			InputPtr += Context.PerInstanceFnInstanceIdx * AdvanceOffset;

			StartPtr = InputPtr;
		}

		inline bool IsConstant() const { return !IsRegister(); }
		inline bool IsRegister() const { return AdvanceOffset > 0; }
		inline void Reset(){ InputPtr = StartPtr; }

		inline const T Get() { return *InputPtr; }
		inline const T* GetDest() { return InputPtr; }
		inline void Advance(int32 Count=1) { InputPtr += AdvanceOffset * Count; }
		inline const T GetAndAdvance()
		{
			const T* Ret = InputPtr;
			InputPtr += AdvanceOffset;
			return *Ret;
		}
		inline const T* GetDestAndAdvance()
		{
			const T* Ret = InputPtr;
			InputPtr += AdvanceOffset;
			return Ret;
		}
	};
	
	template<typename T>
	struct FExternalFuncRegisterHandler
	{
	private:
		T* RESTRICT Register = nullptr;
		int32 AdvanceOffset = 0;
	public:
		FExternalFuncRegisterHandler(FVectorVMExternalFunctionContext& Context)
		{
			Register = (T*)Context.GetNextRegister(&AdvanceOffset);

			//Hack: Offset into the buffer by the instance offset.
			Register += Context.PerInstanceFnInstanceIdx * AdvanceOffset;
		}

		inline bool IsValid() const { return AdvanceOffset > 0; }
		inline const T Get() { return *Register; }
		inline T* GetDest() { return Register; }
		inline void Advance() { Register += AdvanceOffset; }
		inline void Advance(int32 Count) { Register += AdvanceOffset * Count; }
		inline const T GetAndAdvance()
		{
			T* Ret = Register;
			Register += AdvanceOffset;
			return *Ret;
		}
		inline T* GetDestAndAdvance()
		{
			T* Ret = Register;
			Register += AdvanceOffset;
			return Ret;
		}
	};

	template<typename T>
	struct FExternalFuncConstHandler
	{
		T Constant;

		FExternalFuncConstHandler(FVectorVMExternalFunctionContext& Context)
		{
			int32 AdvanceOffset;
			const T* Register = (const T*)Context.GetNextRegister(&AdvanceOffset);
			Register += Context.PerInstanceFnInstanceIdx * AdvanceOffset;

			Constant = *Register;
		}

		VM_FORCEINLINE const T& Get() { return Constant; }
		VM_FORCEINLINE const T& GetAndAdvance() { return Constant; }
		VM_FORCEINLINE void Advance() { }
	};

	enum EVectorVMFlags
	{
		VVMFlag_OptSaveIntermediateState = 1 << 0,
		VVMFlag_OptOmitStats = 1 << 1,
		VVMFlag_LargeScript = 1 << 2,   //if true register indices are 16 bit, otherwise they're 8 bit
		VVMFlag_HasRandInstruction = 1 << 3,
		VVMFlag_DataMapCacheSetup = 1 << 4,
	};
} // namespace VectorVM

