// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "VectorVM.h"

namespace VectorVM
{

#if WITH_EDITORONLY_DATA
namespace Optimizer
{
	struct FVectorVMOptimizerContext;
}
#endif // WITH_EDITORONLY_DATA

namespace Runtime
{
	struct FVectorVMRuntimeContext;
}

namespace Bridge
{

#if WITH_EDITORONLY_DATA
	VECTORVM_API void FreezeOptimizerContext(const Optimizer::FVectorVMOptimizerContext& Context, TArray<uint8>& ContextData);
#endif // WITH_EDITORONLY_DATA

	VECTORVM_API void ThawRuntimeContext(TConstArrayView<uint8> ContextData, Runtime::FVectorVMRuntimeContext& Context);

} // VectorVM::Bridge

} // VectorVM