// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once	

// HEADER_UNIT_SKIP - Bad header

#include "VectorVMDataObject.generated.h"

//TODO Data Objects are to be totally re-worked