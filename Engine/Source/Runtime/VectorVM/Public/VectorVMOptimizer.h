// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "VectorVM.h"

#if WITH_EDITORONLY_DATA

struct FVectorVMExtFunctionData;

namespace VectorVM::Optimizer
{
	struct FVectorVMOptimizerContext;

	VECTORVM_API FVectorVMOptimizerContext* AllocOptimizerContext();
	VECTORVM_API void FreeOptimizerContext(FVectorVMOptimizerContext* Context);

	VECTORVM_API uint32 OptimizeVectorVMScript(const uint8* Bytecode, int BytecodeLen, FVectorVMExtFunctionData* ExtFnIOData, int NumExtFns, FVectorVMOptimizerContext* OptContext, uint64 HashId, uint32 Flags); //OutContext must be zeroed except the Init struct
	VECTORVM_API void GenerateHumanReadableScript(const FVectorVMOptimizerContext& Context, FString& VMScript);

} // VectorVM

#endif
