// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "VectorVM.h"

namespace VectorVM::Runtime
{
	struct FVectorVMState;

	struct FVectorVMExecContext
	{
		struct
		{
			uint32          NumBytesRequiredPerBatch;
			uint32          PerBatchRegisterDataBytesRequired;
			uint32          MaxChunksPerBatch;
			uint32          MaxInstancesPerChunk;
		} Internal;

		FVectorVMState* VVMState;               //created with AllocVectorVMState()
		TArrayView<FDataSetMeta>                DataSets;
		TArrayView<const FVMExternalFunction*> ExtFunctionTable;
		TArrayView<void*>                      UserPtrTable;
		int32                                   NumInstances;
		const uint8* const* ConstantTableData;      //constant tables consist of an array of pointers
		const int* ConstantTableNumBytes;  //an array of sizes in bytes
		int32                                   ConstantTableCount;     //how many constant tables.  These tables must match the ones used with OptimizeVectorVMScript()
	};

	VECTORVM_API FVectorVMState* AllocVectorVMState(TConstArrayView<uint8> ContextData);
	VECTORVM_API void FreeVectorVMState(FVectorVMState* State);

	VECTORVM_API void ExecVectorVMState(FVectorVMExecContext* ExecCtx);
}
