// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class VectorVM : ModuleRules
{
    public VectorVM(ReadOnlyTargetRules Target) : base(Target)
    {
        PublicDependencyModuleNames.AddRange(
            new string[] {
                "Core",
                "CoreUObject"
            }
        );
    }
}
